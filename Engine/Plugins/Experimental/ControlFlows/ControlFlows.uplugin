{
	"FileVersion": 3,
	"Version": 1,
	"VersionName": "1.0",
	"FriendlyName": "ControlFlows",
	"Description": "Tool to cleanly implement Asynchronous Operations",
	"Category": "Other",
	"CreatedBy": "Epic Games, Inc.",
	"CreatedByURL": "https://epicgames.com",
	"DocsURL": "",
	"MarketplaceURL": "",
	"SupportURL": "",
	"CanContainContent": true,
	"IsBetaVersion": false,
	"IsExperimentalVersion": false,
	"Installed": false,
	"SupportedPrograms": [ "LiveLinkHub" ],
	"Modules": [
		{
			"Name": "ControlFlows",
			"Type": "Runtime",
			"LoadingPhase": "Default"
		}
	]
}