// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;

public class ControlFlows : ModuleRules
{
	public ControlFlows(ReadOnlyTargetRules Target) : base(Target)
	{
		PrivateDependencyModuleNames.AddRange(
			new string[]
			{
				"Core",
				"CoreUObject",
				// ... add private dependencies that you statically link with here ...	
			}
			);
	}
}
