// Copyright Epic Games, Inc. All Rights Reserved.

#include "ControlFlow.h"
#include "ControlFlowManager.h"
#include "ControlFlowTask.h"
#include "ControlFlowTask_Loop.h"
#include "ControlFlows.h"
#include "Containers/Ticker.h"
#include "Misc/App.h"
#include "Misc/TrackedActivity.h"

static const int32 MAX_FLOW_LOOPS = 10000;

int32 FControlFlow::UnnamedControlFlowCounter = 0;

FControlFlow::FControlFlow(const FString& FlowDebugName)
{
	if (FlowDebugName.IsEmpty())
	{
		DebugName = FString::Format(TEXT("ControlFlow_{0}"), { FString::FormatAsNumber(UnnamedControlFlowCounter) });
		UnnamedControlFlowCounter++;
	}
	else
	{
		DebugName = FlowDebugName;
	}
}

void FControlFlow::ExecuteNextNodeInQueue()
{
	UE_LOGF(LogControlFlows, Verbose, "ControlFlow - Executing %ls (ExecuteNextInQueue)", *GetFlowPath().Append(DebugName));

	CurrentNode = FlowQueue[0];
	if (Activity)
	{
		Activity->Update(*CurrentNode->GetNodeName());
	}

	FlowQueue.RemoveAt(0);
	CurrentNode->Execute();
}

void FControlFlow::ExecuteNode(TSharedRef<FControlFlowNode_SelfCompleting> SelfCompletingNode)
{
	// Calling Synchronous Function in Flow
	if (SelfCompletingNode->Process.IsBound())
	{
		SelfCompletingNode->Process.Execute();
	}
	else
	{
		SelfCompletingNode->bWasBoundOnExecution = false;
	}
	
	if (SelfCompletingNode->HasCancelBeenRequested())
	{
		FlowQueue.Reset();

		BroadcastCancellation();
		FControlFlowStatics::HandleControlFlowFinishedNotification();
	}
	else
	{
		SelfCompletingNode->ContinueFlow();
	}
}

void FControlFlow::BroadcastCancellation()
{
	OnFlowCancelledDelegate.Broadcast();
	OnCancelledDelegate_Internal.ExecuteIfBound();
}

void FControlFlow::HandleControlFlowNodeCompleted(const TSharedRef<const FControlFlowNode>& NodeCompleted)
{
	UE_LOGF(LogControlFlows, Verbose, "ControlFlow - Executing %ls%ls(x%d).%ls (FlowControlNodeCompleted)", *GetFlowPath(), *DebugName, GetRepeatedFlowCount(), *NodeCompleted->GetNodeName());

#if CPUPROFILERTRACE_ENABLED
	if (bProfilerEventStarted)
	{
		FCpuProfilerTrace::OutputEndEvent();
		bProfilerEventStarted = false;
	}
#endif 

	SubFlowStack_ForDebugging.Add(SharedThis(this));

	if (ensureMsgf(CurrentNode.IsValid(), TEXT("CurrentNode isn't valid when handling control flow node (%s) being completed.  This will likely become a static_assert/compile-error"),
			*NodeCompleted->GetNodeName()
				  ) &&
		ensureMsgf(&NodeCompleted.Get() == CurrentNode.Get(), TEXT("NodeCompleted (%s) does not match the current node (%s).  This will likely become a static_assert/compile-error"),
			*NodeCompleted->GetNodeName(), *CurrentNode->GetNodeName()
				  )
	   )
	{
		if (!NodeCompleted->bWasBoundOnExecution)
		{
			OnNodeWasNotBoundedOnExecution_Internal.ExecuteIfBound();
		}

		const bool bCancelRequested = !bInterpretCancelledNodeAsComplete && NodeCompleted->HasCancelBeenRequested();
		CurrentNode.Reset();
		CurrentlyRunningTask.Reset();

		if (FlowQueue.Num() > 0)
		{
			if (!ensure(SubFlowStack_ForDebugging.Num() < MAX_FLOW_LOOPS))
			{
				UE_LOGF(LogControlFlows, Error, "ControlFlow - Hit maximum Flow loops. Is there an infinite recursion somewhere?");
			}
			else
			{
				if (bCancelRequested)
				{
					CurrentNode = FlowQueue[0];
					if (Activity)
						Activity->Update(*CurrentNode->GetNodeName());

					FlowQueue.RemoveAt(0);

					/* Calling CancelFlow() on the next node will call CancelFlow() through the entire `FlowQueue` nodes because 
					* Calling `FControlFlowNode::CancelFlow()` will call back into `FControlFlow::HandleControlFlowNodeCompleted()` and end here until `FlowQueue.Num() == 0` */
					CurrentNode->CancelFlow(); 
				}
				else
				{
					bBroadcastingStepComplete = true;
					OnStepCompletedDelegate.Broadcast();
					bBroadcastingStepComplete = false;
					ExecuteNextNodeInQueue();
				}
			}
		}
		else
		{
			if (bCancelRequested)
			{
				BroadcastCancellation();
			}
			else
			{
				OnStepCompletedDelegate.Broadcast();
				OnFlowCompleteDelegate.Broadcast();
				OnCompleteDelegate_Internal.ExecuteIfBound();				
			}

			FControlFlowStatics::HandleControlFlowFinishedNotification();
		}
	}

	SubFlowStack_ForDebugging.Pop();
}

FString FControlFlow::GetFlowPath() const
{
	static FString DuplicateFlowFormat = TEXT("{0}(x{1}).");

	FString FlowPath;

	int32 DuplicateCount = 0;

	for (int32 StackIndex = 0; StackIndex < SubFlowStack_ForDebugging.Num() - 1; ++StackIndex)
	{
		int32 NextInStackIndex = StackIndex + 1;

		if (SubFlowStack_ForDebugging[StackIndex] == SubFlowStack_ForDebugging[NextInStackIndex])
		{
			++DuplicateCount;

			if (NextInStackIndex == SubFlowStack_ForDebugging.Num())
			{
				FlowPath.Append(FString::Format(*DuplicateFlowFormat, { SubFlowStack_ForDebugging[StackIndex]->DebugName , FString::FromInt(DuplicateCount + 1) }));
			}
		}
		else
		{
			FlowPath.Append(FString::Format(*DuplicateFlowFormat, { SubFlowStack_ForDebugging[StackIndex]->DebugName , FString::FromInt(DuplicateCount + 1) }));

			DuplicateCount = 0;
		}
	}

	return FlowPath;
}

int32 FControlFlow::GetRepeatedFlowCount() const
{
	int32 FlowCount = 0;

	for (int32 StackIndex = SubFlowStack_ForDebugging.Num() - 1; StackIndex > -1; --StackIndex)
	{
		if (SubFlowStack_ForDebugging[StackIndex] == SharedThis(this))
		{
			++FlowCount;
		}
		else
		{
			break;
		}
	}

	return FlowCount;
}

void FControlFlow::LogNodeExecution(const FControlFlowNode& NodeExecuted)
{
	UE_LOGF(LogControlFlows, Verbose, "ControlFlow - Executing %ls%ls(x%d).%ls", *GetFlowPath(), *DebugName, GetRepeatedFlowCount(), *NodeExecuted.GetNodeName());
}

void FControlFlow::ExecuteFlow()
{
	UE_LOGF(LogControlFlows, Verbose, "ControlFlow - Executing %ls (ExecuteFlow)", *DebugName);

	if (ensureAlwaysMsgf(!CurrentNode.IsValid(), TEXT("Flow is already running! Or perhaps there are multiple instances of owning class? All flows should have a unique ID.")))
	{
		if (!ensure(SubFlowStack_ForDebugging.Num() < MAX_FLOW_LOOPS))
		{
			UE_LOGF(LogControlFlows, Error, "ControlFlow - Hit maximum Flow loops. Is there an infinite recursion somewhere?");
		}
		else
		{
			FControlFlowStatics::HandleControlFlowStartedNotification(AsShared());

			SubFlowStack_ForDebugging.Add(SharedThis(this));
			if (FlowQueue.Num() > 0)
			{
				ExecuteNextNodeInQueue();
			}
			else
			{
				// No nodes is treated as completed for non internal cases
				OnFlowCompleteDelegate.Broadcast();
				OnExecutedWithoutAnyNodesDelegate_Internal.ExecuteIfBound();
				FControlFlowStatics::HandleControlFlowFinishedNotification();
			}

			SubFlowStack_ForDebugging.Pop();
		}
	}

	UE_LOGF(LogControlFlows, Verbose, "ControlFlow - Executing %ls (ExecuteFlow Completed)", *DebugName);
}

void FControlFlow::Reset()
{
	if (CurrentNode.IsValid())
	{
		//Flow is already active. perhaps fire an interrupted delegate?
	}

	CurrentNode.Reset();
	CurrentlyRunningTask.Reset();
	FlowQueue.Reset();
	SubFlowStack_ForDebugging.Reset();
	UnnamedNodeCounter = 0;
	UnnamedBranchCounter = 0;
}

bool FControlFlow::IsRunning() const
{
	return CurrentNode.IsValid() || bBroadcastingStepComplete;
}

void FControlFlow::CancelFlow()
{
	if (CurrentNode.IsValid())
	{
		CurrentNode->CancelFlow();
	}
	else
	{
		BroadcastCancellation();
		FControlFlowStatics::HandleControlFlowFinishedNotification();
	}
}

FControlFlow& FControlFlow::SetCancelledNodeAsComplete(bool bCancelledNodeIsComplete)
{
	bInterpretCancelledNodeAsComplete = bCancelledNodeIsComplete;
	return *this;
}

FString FControlFlow::FormatOrGetNewNodeDebugName(const FString& FlowNodeDebugName)
{
	if (FlowNodeDebugName.IsEmpty())
	{
		UnnamedNodeCounter++;
		return FString::Format(TEXT("UnnamedNode_{0}"), { FString::FromInt(UnnamedNodeCounter - 1) });
	}

	return FlowNodeDebugName;
}

TOptional<FString> FControlFlow::GetCurrentStepDebugName() const
{
	if (CurrentNode.IsValid())
	{
		if (CurrentlyRunningTask.IsValid())
		{
			return CurrentlyRunningTask->GetNodeName();
		}
		
		return CurrentNode->GetNodeName();
	}

	return TOptional<FString>();
}

TSharedPtr<FTrackedActivity> FControlFlow::GetTrackedActivity() const
{
	return Activity;
}

FControlFlow& FControlFlow::QueueDelay(const float InSeconds, const FString& NodeName)
{
	QueueWait(NodeName).BindLambda([InSeconds](FControlFlowNodeRef FlowHandle)
		{
			TSharedRef<FControlFlow> OwningFlow = FlowHandle->Parent.Pin().ToSharedRef();

			// If we already had a 0.0f delay this tick, add the delta time of this tick to the delay
			const float AdditionalDelay = (InSeconds == 0.0f && OwningFlow->LastZeroSecondDelay.Key == FApp::GetCurrentTime()) ? OwningFlow->LastZeroSecondDelay.Value : 0.0f;
			const float SecondsToDelay = InSeconds + AdditionalDelay;

			FTSTicker::GetCoreTicker().AddTicker(FTickerDelegate::CreateLambda([FlowHandle, bIsZeroSecondDelay = SecondsToDelay == 0.0f](float DeltaTime)
				{
					if (FlowHandle->Parent.IsValid() && !FlowHandle->HasCancelBeenRequested())
					{
						// Notify ALL parents of 0.0f delay
						if (bIsZeroSecondDelay)
						{
							TSharedPtr<FControlFlow> FlowIter = FlowHandle->Parent.Pin().ToSharedRef();
							while (FlowIter.IsValid())
							{
								FlowIter->LastZeroSecondDelay = { FApp::GetCurrentTime(), DeltaTime };

								FlowIter = FlowIter->ParentFlow.IsValid() ? FlowIter->ParentFlow.Pin() : nullptr;
							}
						}

						FlowHandle->ContinueFlow();
					}
					return false;
				}), SecondsToDelay);
		});

	return *this;
}

FControlFlow& FControlFlow::QueueSetCancelledNodeAsComplete(const bool bCancelledNodeIsComplete, const FString& NodeName /*= FString()*/)
{
	QueueFunction(NodeName).BindSPLambda(this, [this, bCancelledNodeIsComplete]()
		{
			this->SetCancelledNodeAsComplete(bCancelledNodeIsComplete);
		});

	return *this;
}

FControlFlow& FControlFlow::TrackActivities(TSharedPtr<FTrackedActivity> InActivity)
{
	if (!InActivity)
		InActivity = MakeShared<FTrackedActivity>(*DebugName, TEXT(""));
	Activity = InActivity;
	return *this;
}

FSimpleDelegate& FControlFlow::QueueFunction(const FString& FlowNodeDebugName)
{
	TSharedRef<FControlFlowNode_SelfCompleting> NewNode = MakeShared<FControlFlowNode_SelfCompleting>(SharedThis(this), FormatOrGetNewNodeDebugName(FlowNodeDebugName));
	FlowQueue.Add(NewNode);
	
	return NewNode->Process;
}

FControlFlowWaitDelegate& FControlFlow::QueueWait(const FString& FlowNodeDebugName)
{
	TSharedRef<FControlFlowNode_RequiresCallback> NewNode = MakeShared<FControlFlowNode_RequiresCallback>(SharedThis(this), FormatOrGetNewNodeDebugName(FlowNodeDebugName));
	FlowQueue.Add(NewNode);

	return NewNode->Process;
}

FControlFlowPopulator& FControlFlow::QueueControlFlow(const FString& TaskName /*= TEXT("")*/, const FString& FlowNodeDebugName /*= TEXT("")*/)
{
	TSharedRef<FControlFlowSimpleSubTask> NewTask = MakeShared<FControlFlowSimpleSubTask>(TaskName);
	TSharedRef<FControlFlowNode_Task> NewNode = MakeShared<FControlFlowNode_Task>(SharedThis(this), NewTask, FormatOrGetNewNodeDebugName(FlowNodeDebugName));

	NewTask->GetTaskFlow()->ParentFlow = AsWeak();

	NewTask->SetOwningNode(NewNode);
	NewTask->GetTaskFlow()->Activity = Activity;
	NewNode->OnExecute().BindSP(SharedThis(this), &FControlFlow::HandleTaskNodeExecuted);
	NewNode->OnCancelRequested().BindSP(SharedThis(this), &FControlFlow::HandleTaskNodeCancelled);

	FlowQueue.Add(NewNode);

	return NewTask->GetTaskPopulator();
}

FControlFlowBranchDefiner& FControlFlow::QueueControlFlowBranch(const FString& TaskName /*= TEXT("")*/, const FString& FlowNodeDebugName /*= TEXT("")*/)
{
	TSharedRef<FControlFlowTask_Branch> NewTask = MakeShared<FControlFlowTask_Branch>(TaskName);
	TSharedRef<FControlFlowNode_Task> NewNode = MakeShared<FControlFlowNode_Task>(SharedThis(this), NewTask, FormatOrGetNewNodeDebugName(FlowNodeDebugName));

	NewTask->SetOwningNode(NewNode);
	NewTask->Activity = Activity;
	NewNode->OnExecute().BindSP(SharedThis(this), &FControlFlow::HandleTaskNodeExecuted);
	NewNode->OnCancelRequested().BindSP(SharedThis(this), &FControlFlow::HandleTaskNodeCancelled);

	FlowQueue.Add(NewNode);

	return NewTask->GetDelegate();
}

FConcurrentFlowsDefiner& FControlFlow::QueueConcurrentFlows(const FString& TaskName /*= TEXT("")*/, const FString& FlowNodeDebugName /*= TEXT("")*/)
{
	TSharedRef<FControlFlowTask_ConcurrentFlows> NewTask = MakeShared<FControlFlowTask_ConcurrentFlows>(TaskName);
	TSharedRef<FControlFlowNode_Task> NewNode = MakeShared<FControlFlowNode_Task>(SharedThis(this), NewTask, FormatOrGetNewNodeDebugName(FlowNodeDebugName));

	NewTask->SetOwningNode(NewNode);
	NewNode->OnExecute().BindSP(SharedThis(this), &FControlFlow::HandleTaskNodeExecuted);
	NewNode->OnCancelRequested().BindSP(SharedThis(this), &FControlFlow::HandleTaskNodeCancelled);

	FlowQueue.Add(NewNode);

	return NewTask->GetDelegate();
}

FControlFlowConditionalLoopDefiner& FControlFlow::QueueConditionalLoop(const FString& TaskName /*= TEXT("")*/, const FString& FlowNodeDebugName /*= TEXT("")*/)
{
	TSharedRef<FControlFlowTask_ConditionalLoop> NewTask = MakeShared<FControlFlowTask_ConditionalLoop>(TaskName);
	TSharedRef<FControlFlowNode_Task> NewNode = MakeShared<FControlFlowNode_Task>(SharedThis(this), NewTask, FormatOrGetNewNodeDebugName(FlowNodeDebugName));

	NewTask->SetOwningNode(NewNode);
	NewNode->OnExecute().BindSP(SharedThis(this), &FControlFlow::HandleTaskNodeExecuted);
	NewNode->OnCancelRequested().BindSP(SharedThis(this), &FControlFlow::HandleTaskNodeCancelled);

	FlowQueue.Add(NewNode);

	return NewTask->GetDelegate();
}

TSharedRef<FControlFlowTask_BranchLegacy> FControlFlow::QueueBranch(FControlFlowBranchDecider_Legacy& BranchDecider, const FString& TaskName /*= TEXT("")*/, const FString& FlowNodeDebugName /*= TEXT("")*/)
{
	ensureAlwaysMsgf(false, TEXT("Deprecated. Use 'QueueControlFlowBranch'"));

	TSharedRef<FControlFlowTask_BranchLegacy> NewTask = MakeShared<FControlFlowTask_BranchLegacy>(BranchDecider, TaskName);
	TSharedRef<FControlFlowNode_Task> NewNode = MakeShared<FControlFlowNode_Task>(SharedThis(this), NewTask, FormatOrGetNewNodeDebugName(FlowNodeDebugName));

	NewTask->SetOwningNode(NewNode);
	NewNode->OnExecute().BindSP(SharedThis(this), &FControlFlow::HandleTaskNodeExecuted);
	NewNode->OnCancelRequested().BindSP(SharedThis(this), &FControlFlow::HandleTaskNodeCancelled);

	FlowQueue.Add(NewNode);

	return NewTask;
}

FControlFlowPopulator& FControlFlow::QueueLoop(FControlFlowLoopComplete& LoopCompleteDelgate, const FString& TaskName /*= TEXT("")*/, const FString& FlowNodeDebugName /*= TEXT("")*/)
{
	TSharedRef<FControlFlowTask_LoopDeprecated> NewTask = MakeShared<FControlFlowTask_LoopDeprecated>(LoopCompleteDelgate, TaskName);
	TSharedRef<FControlFlowNode_Task> NewNode = MakeShared<FControlFlowNode_Task>(SharedThis(this), NewTask, FormatOrGetNewNodeDebugName(FlowNodeDebugName));

	NewTask->GetTaskFlow()->ParentFlow = AsWeak();

	NewTask->SetOwningNode(NewNode);
	NewTask->GetTaskFlow()->Activity = Activity;
	NewNode->OnExecute().BindSP(SharedThis(this), &FControlFlow::HandleTaskNodeExecuted);
	NewNode->OnCancelRequested().BindSP(SharedThis(this), &FControlFlow::HandleTaskNodeCancelled);

	FlowQueue.Add(NewNode);

	return NewTask->GetTaskPopulator();
}

void FControlFlow::HandleTaskNodeExecuted(TSharedRef<FControlFlowNode_Task> TaskNode)
{
	UE_LOGF(LogControlFlows, Verbose, "ControlFlow - Executing %ls%ls (ExecuteTask)", *GetFlowPath(), *DebugName);

	if (ensure(CurrentNode.IsValid() && CurrentNode == TaskNode && !CurrentlyRunningTask.IsValid()))
	{
		CurrentlyRunningTask = TaskNode;
		CurrentNode = CurrentlyRunningTask;
		if (Activity)
		{
			Activity->Update(*TaskNode->GetFlowTask()->DebugName);
		}

		TaskNode->GetFlowTask()->OnComplete().BindSP(SharedThis(this), &FControlFlow::HandleOnTaskComplete);
		TaskNode->GetFlowTask()->OnCancelled().BindSP(SharedThis(this), &FControlFlow::HandleOnTaskCancelled);

		TaskNode->GetFlowTask()->Execute();
	}
}

void FControlFlow::HandleTaskNodeCancelled(TSharedRef<FControlFlowNode_Task> TaskNode)
{
	CurrentlyRunningTask = TaskNode;

	TaskNode->GetFlowTask()->OnComplete().BindSP(SharedThis(this), &FControlFlow::HandleOnTaskComplete);
	TaskNode->GetFlowTask()->OnCancelled().BindSP(SharedThis(this), &FControlFlow::HandleOnTaskCancelled);

	TaskNode->GetFlowTask()->Cancel();
}

void FControlFlow::HandleOnTaskComplete()
{
	UE_LOGF(LogControlFlows, Verbose, "ControlFlow - Executing %ls%ls (TaskComplete)", *GetFlowPath(), *DebugName);

	if (CurrentlyRunningTask.IsValid())
	{
		CurrentlyRunningTask->ContinueFlow();
	}
}

void FControlFlow::HandleOnTaskCancelled()
{
	UE_LOGF(LogControlFlows, Verbose, "ControlFlow - Cancelling %ls%ls (TaskCancelled)", *GetFlowPath(), *DebugName);

	if (CurrentlyRunningTask.IsValid())
	{
		CurrentlyRunningTask->CompleteCancelFlow();
	}
}
