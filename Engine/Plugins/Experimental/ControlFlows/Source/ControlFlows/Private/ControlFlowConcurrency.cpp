// Copyright Epic Games, Inc. All Rights Reserved.

#include "ControlFlowConcurrency.h"

#include "Containers/Ticker.h"
#include "ControlFlow.h"
#include "ControlFlows.h"
#include "ControlFlowTask.h"
#include "Async/ParallelFor.h"

FConcurrencySubFlowContainer::FConcurrencySubFlowContainer(const FString& InDebugName)
	: SubFlow(MakeShared<FControlFlow>(InDebugName))
{}

bool FConcurrencySubFlowContainer::HasBeenExecuted() const
{
	return bHasBeenExecuted;
}

bool FConcurrencySubFlowContainer::HasBeenCancelled() const
{
	return bHasBeenCancelled;
}

bool FConcurrencySubFlowContainer::IsCompleteOrCancelled() const
{
	const bool bIsComplete = HasBeenExecuted() && !SubFlow->IsRunning();
	return bIsComplete || HasBeenCancelled();
}

void FConcurrencySubFlowContainer::Execute(TWeakPtr<FConcurrentControlFlows::FParallelFlowLock> InFlowLock)
{
	UE_LOGF(LogControlFlows, Verbose, "ConcurrentControlFlowContainer - Executing %ls", *GetDebugName());

	if (ensureAlways(!IsCompleteOrCancelled()))
	{
		ParallelFlowLock = InFlowLock.IsValid() ? InFlowLock.Pin() : nullptr;
		bHasBeenExecuted = true;
		SubFlow->ExecuteFlow();
	}
}

void FConcurrencySubFlowContainer::Cancel()
{
	UE_LOGF(LogControlFlows, Verbose, "ConcurrentControlFlowContainer - Cancelling %ls", *GetDebugName());

	bHasBeenCancelled = true;
	if (SubFlow->IsRunning())
	{
		SubFlow->CancelFlow();
	}
}

FSimpleDelegate& FConcurrencySubFlowContainer::OnComplete()
{
	return SubFlow->OnCompleteDelegate_Internal;
}

FSimpleDelegate& FConcurrencySubFlowContainer::OnExecutedWithoutAnyNodes()
{
	return SubFlow->OnExecutedWithoutAnyNodesDelegate_Internal;
}

FSimpleDelegate& FConcurrencySubFlowContainer::OnCancelled()
{
	return SubFlow->OnCancelledDelegate_Internal;
}

const FString& FConcurrencySubFlowContainer::GetDebugName()
{
	return SubFlow->GetDebugName();
}

TSharedRef<FControlFlow> FConcurrencySubFlowContainer::GetControlFlow() const
{
	return SubFlow;
}

////////////////////////////////////////////////////////////////////////////////////////

FControlFlow& FConcurrentControlFlows::AddOrGetFlow(int32 InIdentifier, const FString& DebugSubFlowName /*= TEXT("")*/)
{
	if (TSharedRef<FConcurrencySubFlowContainer>* ExistingFlow = ConcurrentFlows.Find(InIdentifier))
	{
		return (*ExistingFlow)->GetControlFlow().Get();
	}
	else
	{
		static int32 UnnamedControlConcurrencyFlowCounter = 0;
		const FString& DebugNameToUse = DebugSubFlowName.IsEmpty() ? FString::Format(TEXT("UnnamedConcurrencySubFlow_{0}"), { (UnnamedControlConcurrencyFlowCounter++) }) : DebugSubFlowName;

		TSharedRef<FConcurrencySubFlowContainer> NewConcurrentFlow = MakeShared<FConcurrencySubFlowContainer>(DebugNameToUse);

		check(OwningTask.IsValid());

		NewConcurrentFlow->GetControlFlow()->ParentFlow = OwningTask.Pin()->GetOwningFlowForTaskNode();

		NewConcurrentFlow->OnComplete().BindSP(SharedThis(this), &FConcurrentControlFlows::HandleConcurrentFlowDone, InIdentifier, true);
		NewConcurrentFlow->OnExecutedWithoutAnyNodes().BindSP(SharedThis(this), &FConcurrentControlFlows::HandleConcurrentFlowDone, InIdentifier, true);
		NewConcurrentFlow->OnCancelled().BindSP(SharedThis(this), &FConcurrentControlFlows::HandleConcurrentFlowDone, InIdentifier, false);

		ConcurrentFlows.Add(InIdentifier, NewConcurrentFlow);
		return NewConcurrentFlow->GetControlFlow().Get();
	}
}

FConcurrentControlFlows& FConcurrentControlFlows::SetExecution(const EConcurrentExecution InBehavior)
{
	ExecutionBehavior = InBehavior;
	return *this;
}

FControlFlow& FConcurrentControlFlows::AddOrGetProng(int32 InIdentifier, const FString& DebugSubFlowName /*= TEXT("")*/)
{
	return AddOrGetFlow(InIdentifier, DebugSubFlowName);
}

bool FConcurrentControlFlows::AreAllSubFlowsCompletedOrCancelled() const
{
	for (const TPair<int32, TSharedRef<FConcurrencySubFlowContainer>>& PairIt : ConcurrentFlows)
	{
		if (!PairIt.Value->IsCompleteOrCancelled())
		{
			return false;
		}
	}

	return true;
}

bool FConcurrentControlFlows::HasAnySubFlowBeenExecuted() const
{
	for (const TPair<int32, TSharedRef<FConcurrencySubFlowContainer>>& PairIt : ConcurrentFlows)
	{
		if (PairIt.Value->HasBeenExecuted())
		{
			return true;
		}
	}

	return false;
}

void FConcurrentControlFlows::HandleConcurrentFlowDone(int32 FlowIndex, const bool bCompleted)
{
	check(ConcurrentFlows.Contains(FlowIndex));

	UE_LOGF(LogControlFlows, Verbose, "ConcurrentControlFlow %ls: %ls", bCompleted ? TEXT("Finished") : TEXT("Cancelled"), *ConcurrentFlows[FlowIndex]->GetDebugName());

	if (ExecutionBehavior == EConcurrentExecution::Parallel)
	{
		ConcurrentFlows[FlowIndex].Get().ParallelFlowLock.Reset();
	}
	else if (!bCancelAllHasBegun)
	{
		CheckToBroadcastComplete();
	}
}

void FConcurrentControlFlows::CheckToBroadcastComplete()
{
	if (GetConcurrencyBehavior().GetContinueCondition() == FConcurrentControlFlowBehavior::EContinueConditions::Default)
	{
		if (AreAllSubFlowsCompletedOrCancelled())
		{
			OnConcurrencyCompleted.ExecuteIfBound();
		}
		else
		{
			UE_LOGF(LogControlFlows, Verbose, "Other flows are still running");
		}
	}
	else
	{
		checkf(false, TEXT("Unhandled Continue Condition"));
	}
}

void FConcurrentControlFlows::Execute()
{
	ensureAlwaysMsgf(!this->HasAnySubFlowBeenExecuted(), TEXT("Did you call ExecuteFlow() on a SubFlow? Do not do this! You only need to call ExecuteFlow once per FControlFlowStatics::Create!"));

	if (ConcurrentFlows.Num() == 0)
	{
		OnAllCompleted();
		return;
	}

	TArray<TSharedRef<FConcurrencySubFlowContainer>> FlowsToExecute;
	for (const TPair<int32, TSharedRef<FConcurrencySubFlowContainer>>& PairIt : ConcurrentFlows)
	{
		FlowsToExecute.Add(PairIt.Value);
		PairIt.Value->GetControlFlow()->LastZeroSecondDelay = OwningTask.Pin()->GetOwningFlowForTaskNode().Pin()->LastZeroSecondDelay;
	}

	if (ExecutionBehavior == EConcurrentExecution::Parallel)
	{
		ParallelFlowLock = MakeShared<FConcurrentControlFlows::FParallelFlowLock, ESPMode::ThreadSafe>();

		ParallelFor(FlowsToExecute.Num(), [this, &FlowsToExecute](int32 Index)
			{
				UE_LOGF(LogControlFlows, Verbose, "ConcurrentControlFlow::Execute - Executing Subflow %ls", *FlowsToExecute[Index]->GetDebugName());
				FlowsToExecute[Index]->Execute(ParallelFlowLock);
			});

		if (ParallelFlowLock.IsUnique())
		{
			ensure(AreAllSubFlowsCompletedOrCancelled());
			CheckToBroadcastComplete();
		}
		else
		{
			FTSTicker::GetCoreTicker().AddTicker(FTickerDelegate::CreateSPLambda(this, [this](float)
				{
					if (ParallelFlowLock.IsUnique())
					{
						ensure(AreAllSubFlowsCompletedOrCancelled());
						CheckToBroadcastComplete();
						return false;
					}
					else
					{
						return true;
					}
				}), 0.0f);
		}
	}
	else
	{
		if (ExecutionBehavior == EConcurrentExecution::Random)
		{
			for (int32 Idx = 0; Idx < FlowsToExecute.Num(); ++Idx)
			{
				Swap(FlowsToExecute[Idx], FlowsToExecute[FMath::RandRange(0, FlowsToExecute.Num() - 1)]);
			}
		}

		for (const TSharedRef<FConcurrencySubFlowContainer>& Flow : FlowsToExecute)
		{
			UE_LOGF(LogControlFlows, Verbose, "ConcurrentControlFlow::Execute - Executing Subflow %ls", *Flow->GetDebugName());

			Flow->Execute(nullptr);

			if (bCancelAllHasBegun)
			{
				break;
			}
		}
	}
}

void FConcurrentControlFlows::CancelAll()
{
	if (!bCancelAllHasBegun)
	{
		UE_LOGF(LogControlFlows, Verbose, "FConcurrentControlFlows::CancelAll() called");

		bCancelAllHasBegun = true;

		for (TPair<int32, TSharedRef<FConcurrencySubFlowContainer>>& PairIt : ConcurrentFlows)
		{
			UE_LOGF(LogControlFlows, Verbose, "ConcurrentControlFlow::Cancel All - Cancelling Subflow %ls", *PairIt.Value->GetDebugName());

			PairIt.Value->Cancel();
		}

		OnAllCancelled();
	}
}

void FConcurrentControlFlows::OnAllCompleted()
{
	UE_LOGF(LogControlFlows, Verbose, "ConcurrentControlFlow::OnAllCompleted()");

	ConcurrentFlows.Reset();
	OnConcurrencyCompleted.ExecuteIfBound();
}

void FConcurrentControlFlows::OnAllCancelled()
{
	ConcurrentFlows.Reset();
	OnConcurrencyCancelled.ExecuteIfBound();
}

FConcurrentControlFlowBehavior FConcurrentControlFlows::GetConcurrencyBehavior() const
{
	return FConcurrentControlFlowBehavior();
}
