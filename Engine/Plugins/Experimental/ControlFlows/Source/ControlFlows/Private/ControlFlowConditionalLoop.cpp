// Copyright Epic Games, Inc. All Rights Reserved.

#include "ControlFlowConditionalLoop.h"
#include "ControlFlow.h"

FConditionalLoop::FConditionalLoop()
	: FlowLoop(MakeShared<FControlFlow>())
{}

FControlFlow& FConditionalLoop::SetCheckConditionFirst(bool bInValue)
{
	CheckConditionalFirst.Emplace(bInValue);
	return FlowLoop.Get();
}

FControlFlow& FConditionalLoop::ExecuteAtLeastOnce()
{
	return SetCheckConditionFirst(false);
}

FControlFlow& FConditionalLoop::CheckConditionFirst()
{
	return SetCheckConditionFirst(true);
}

FControlFlow& FConditionalLoop::RunLoopFirst()
{
	return SetCheckConditionFirst(false);
}
