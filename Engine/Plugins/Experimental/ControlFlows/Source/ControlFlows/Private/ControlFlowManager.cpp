// Copyright Epic Games, Inc. All Rights Reserved.

#include "ControlFlowManager.h"
#include "Containers/Ticker.h"
#include "ControlFlows.h"

TArray<TSharedRef<FControlFlowContainerBase>>& FControlFlowStatics::GetNewlyCreatedFlows()
{
	return Get().NewlyCreatedFlows;
}

TArray<TSharedRef<FControlFlowContainerBase>>& FControlFlowStatics::GetPersistentFlows()
{
	return Get().PersistentFlows;
}

TArray<TSharedRef<FControlFlowContainerBase>>& FControlFlowStatics::GetExecutingFlows()
{
	return Get().ExecutingFlows;
}

TArray<TSharedRef<FControlFlowContainerBase>>& FControlFlowStatics::GetFinishedFlows()
{
	return  Get().FinishedFlows;
}

void FControlFlowStatics::HandleControlFlowStartedNotification(TSharedRef<const FControlFlow> InFlow)
{
	TArray<TSharedRef<FControlFlowContainerBase>>& NewFlows = GetNewlyCreatedFlows();
	for (size_t Idx = 0; Idx < NewFlows.Num(); ++Idx)
	{
		if (ensure(UE::Private::OwningObjectIsValid(NewFlows[Idx])))
		{
			if (InFlow == NewFlows[Idx]->GetControlFlow())
			{
				GetExecutingFlows().Add(NewFlows[Idx]);
			}
		}

		NewFlows.RemoveAtSwap(Idx);
		--Idx;
	}

	CheckForInvalidFlows();
}

void FControlFlowStatics::CheckNewlyCreatedFlows()
{
	if (!Get().NextFrameCheckForExecution.IsValid())
	{
		Get().NextFrameCheckForExecution = FTSTicker::GetCoreTicker().AddTicker(FTickerDelegate::CreateStatic(&FControlFlowStatics::IterateThroughNewlyCreatedFlows));
	}
}

void FControlFlowStatics::CheckForInvalidFlows()
{
	if (!Get().NextFrameCheckForFlowCleanup.IsValid())
	{
		Get().NextFrameCheckForFlowCleanup = FTSTicker::GetCoreTicker().AddTicker(FTickerDelegate::CreateStatic(&FControlFlowStatics::IterateForInvalidFlows));
	}
}

bool FControlFlowStatics::IterateThroughNewlyCreatedFlows(float DeltaTime)
{
	Get().NextFrameCheckForExecution.Reset();
	TArray<TSharedRef<FControlFlowContainerBase>>& NewFlows = GetNewlyCreatedFlows();
	for (size_t Idx = 0; Idx < NewFlows.Num(); ++Idx)
	{
		if (ensureAlways(UE::Private::OwningObjectIsValid(NewFlows[Idx])))
		{
			TSharedRef<FControlFlow> NewFlow = NewFlows[Idx]->GetControlFlow();
			if (ensureAlwaysMsgf(NewFlow->IsRunning(), TEXT("Call to execute after queue-ing your steps to avoid this ensure. We will fire the flow 1 frame late to hopefully not cause anything from breaking. Flow:%s"), *NewFlow->GetDebugName()))
			{
				GetExecutingFlows().Add(NewFlows[Idx]);
			}
			else
			{
				if (ensureAlwaysMsgf(NewFlow->NumInQueue() > 0, TEXT("We should never have a newly created flow with no steps. Flow:%s"), *NewFlow->GetDebugName()))
				{
					NewFlow->ExecuteFlow();
					GetExecutingFlows().Add(NewFlows[Idx]);
				}
				else
				{
					GetFinishedFlows().Add(NewFlows[Idx]);
					CheckForInvalidFlows();
				}
			}
		}
	}

	NewFlows.Reset();

	return false;
}

bool FControlFlowStatics::IterateForInvalidFlows(float DeltaTime)
{
	Get().NextFrameCheckForFlowCleanup.Reset();
	//Iterating through Persistent Flows
	{
		TArray<TSharedRef<FControlFlowContainerBase>>& Persistent = GetPersistentFlows();
		for (size_t Idx = 0; Idx < Persistent.Num(); ++Idx)
		{
			if (UE::Private::OwningObjectIsValid(Persistent[Idx]))
			{
				TSharedRef<FControlFlow> PersistentFlow = Persistent[Idx]->GetControlFlow();
				if (PersistentFlow->IsRunning())
				{
					GetExecutingFlows().Add(Persistent[Idx]);
					Persistent.RemoveAtSwap(Idx);
					--Idx;
				}
			}
			else
			{
				Persistent.RemoveAtSwap(Idx);
				--Idx;
			}
		}
	}

	//Iterating through Executing Flows
	{
		TArray<TSharedRef<FControlFlowContainerBase>>& Executing = GetExecutingFlows();
		for (size_t Idx = 0; Idx < Executing.Num(); ++Idx)
		{
			if (UE::Private::OwningObjectIsValid(Executing[Idx]))
			{
				TSharedRef<FControlFlow> ExecutingFlow = Executing[Idx]->GetControlFlow();
				if (!ExecutingFlow->IsRunning() && ensureAlways(ExecutingFlow->NumInQueue() == 0))
				{
					Executing[Idx]->GetControlFlow()->Activity = nullptr;
					GetFinishedFlows().Add(Executing[Idx]);
					Executing.RemoveAtSwap(Idx);
					--Idx;
				}
			}
			else
			{
				Executing[Idx]->GetControlFlow()->Activity = nullptr;
				Executing.RemoveAtSwap(Idx);
				--Idx;
			}
		}
	}

	//Iterating through Completed Flows
	{
		TArray<TSharedRef<FControlFlowContainerBase>>& Completed = GetFinishedFlows();
		for (size_t Idx = 0; Idx < Completed.Num(); ++Idx)
		{
			if (!UE::Private::OwningObjectIsValid(Completed[Idx]))
			{
				UE_LOGF(LogControlFlows, Warning, "Owning Object for completed flow is not valid!");
			}
			
			TSharedRef<FControlFlow> CompletedFlow = Completed[Idx]->GetControlFlow();

			ensureAlwaysMsgf(!CompletedFlow->IsRunning() && CompletedFlow->NumInQueue() == 0, TEXT("Completed Flow (%s) still has items in it's queue"), *CompletedFlow->GetDebugName());

			Completed.RemoveAtSwap(Idx);
			--Idx;
		}
	}

	return false;
}

FControlFlowStatics& FControlFlowStatics::Get()
{
	static TUniquePtr<FControlFlowStatics> Singleton = MakeUnique<FControlFlowStatics>();
	return *Singleton;
}
