// Copyright Epic Games, Inc. All Rights Reserved.

#include "ControlFlowNode.h"

#include "ControlFlows.h"
#include "ControlFlowTask.h"


FControlFlowNode::FControlFlowNode()
	: Parent(nullptr)
{
	
}

FControlFlowNode::FControlFlowNode(TSharedRef<FControlFlow> ControlFlowParent, const FString& FlowNodeDebugName)
	: Parent(ControlFlowParent)
	, NodeName(FlowNodeDebugName)
{}

FControlFlowNode::~FControlFlowNode()
{}

void FControlFlowNode::ContinueFlow()
{
	TSharedPtr<FControlFlow> PinnedParent = Parent.Pin();
	if (ensure(PinnedParent))
	{
		PinnedParent->HandleControlFlowNodeCompleted(SharedThis(this));
	}
}

void FControlFlowNode::CancelFlow()
{
	TSharedPtr<FControlFlow> PinnedParent = Parent.Pin();
	if (ensure(PinnedParent))
	{
		bCancelled = true;
		PinnedParent->HandleControlFlowNodeCompleted(SharedThis(this));
	}
}

TSharedPtr<FTrackedActivity> FControlFlowNode::GetTrackedActivity() const
{
	TSharedPtr<FControlFlow> PinnedParent = Parent.Pin();
	if (ensure(PinnedParent))
	{
		return PinnedParent->GetTrackedActivity();
	}
	return nullptr;
}

void FControlFlowNode::LogExecution()
{
	TSharedPtr<FControlFlow> PinnedParent = Parent.Pin();
	if (ensure(PinnedParent))
	{
		PinnedParent->LogNodeExecution(*this);
	}
}

FString FControlFlowNode::GetNodeName() const
{
	return NodeName;
}

void FControlFlowNode::SetProfilerEventStarted()
{
	TSharedPtr<FControlFlow> PinnedParent = Parent.Pin();
	if (ensure(PinnedParent))
	{
		PinnedParent->SetProfilerEventStarted();
	}
}

///////////////////////////////////////////////////

FControlFlowNode_SelfCompleting::FControlFlowNode_SelfCompleting(TSharedRef<FControlFlow> ControlFlowParent, const FString& FlowNodeDebugName)
	: FControlFlowNode(ControlFlowParent, FlowNodeDebugName)
{
}

FControlFlowNode_SelfCompleting::FControlFlowNode_SelfCompleting(TSharedRef<FControlFlow> ControlFlowParent, const FString& FlowNodeDebugName, const FSimpleDelegate& InCallback)
	: FControlFlowNode(ControlFlowParent, FlowNodeDebugName)
	, Process(InCallback)
{

}

void FControlFlowNode_SelfCompleting::Execute()
{
	LogExecution();

	TSharedPtr<FControlFlow> PinnedParent = Parent.Pin();
	if (ensureAlways(PinnedParent))
	{
		PinnedParent->ExecuteNode(SharedThis(this));
	}
}

///////////////////////////////////////////////////

FControlFlowNode_RequiresCallback::FControlFlowNode_RequiresCallback(TSharedRef<FControlFlow> ControlFlowParent, const FString& FlowNodeDebugName)
	: FControlFlowNode(ControlFlowParent, FlowNodeDebugName)
{
}

FControlFlowNode_RequiresCallback::FControlFlowNode_RequiresCallback(TSharedRef<FControlFlow> ControlFlowParent, const FString& FlowNodeDebugName, const FControlFlowWaitDelegate& InCallback)
	: FControlFlowNode(ControlFlowParent, FlowNodeDebugName)
	, Process(InCallback)
{
}

void FControlFlowNode_RequiresCallback::Execute()
{
	LogExecution();

	if (Process.IsBound())
	{
		TSharedPtr<FControlFlow> PinnedParent = Parent.Pin();
		if (ensureAlways(PinnedParent))
		{
			UE_LOGF(LogControlFlows, Verbose, "ControlFlow - Executing %ls.%ls -  Waiting On Callback", *(PinnedParent->GetFlowPath().Append(PinnedParent->GetDebugName())), *NodeName);
		}

		Process.Execute(SharedThis(this));
	}
	else
	{
		bWasBoundOnExecution = false;
		ContinueFlow();
	}
}

///////////////////////////////////////////////////

FControlFlowNode_Task::FControlFlowNode_Task(TSharedRef<FControlFlow> ControlFlowParent, TSharedRef<FControlFlowSubTaskBase> ControlFlowTask, const FString& FlowNodeDebugName)
	: FControlFlowNode(ControlFlowParent, FlowNodeDebugName)
	, FlowTask(ControlFlowTask)
{
}

void FControlFlowNode_Task::Execute()
{
	LogExecution();

	if (OnExecuteDelegate.IsBound())
	{
		OnExecuteDelegate.Execute(SharedThis(this));
	}
	else
	{
		bWasBoundOnExecution = false;
		ContinueFlow();
	}
}

void FControlFlowNode_Task::CancelFlow()
{
	bCancelled = true;
	OnCancelRequestedDelegate.ExecuteIfBound(SharedThis(this));
}

FString FControlFlowNode_Task::GetNodeName() const
{
	const FString& TaskName = GetFlowTask()->GetTaskName();
	return TaskName.IsEmpty() ? NodeName : TaskName;
}

void FControlFlowNode_Task::CompleteCancelFlow()
{
	FControlFlowNode::CancelFlow();
}