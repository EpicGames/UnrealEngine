// Copyright Epic Games, Inc. All Rights Reserved.

#include "ControlFlowTask.h"
#include "ControlFlows.h"
#include "ControlFlowBranch.h"
#include "ControlFlowConcurrency.h"

//////////////////////////
//FControlFlowTask
//////////////////////////

FControlFlowSubTaskBase::FControlFlowSubTaskBase(const FString& TaskName)
:	DebugName(TaskName)
{

}

TWeakPtr<FControlFlow> FControlFlowSubTaskBase::GetOwningFlowForTaskNode() const
{
	return ensureAlways(OwningNode.IsValid()) ? OwningNode.Pin()->Parent : nullptr;
}

void FControlFlowSubTaskBase::Execute()
{
	TaskCompleteCallback.ExecuteIfBound();
}

void FControlFlowSubTaskBase::Cancel()
{
	TaskCancelledCallback.ExecuteIfBound();
}

//////////////////////////
//FControlFlowBranch
//////////////////////////

FControlFlowTask_BranchLegacy::FControlFlowTask_BranchLegacy(FControlFlowBranchDecider_Legacy& BranchDecider, const FString& TaskName)
	: FControlFlowSubTaskBase(TaskName)
	, BranchDelegate(BranchDecider)
{

}

FSimpleDelegate& FControlFlowTask_BranchLegacy::QueueFunction(int32 BranchIndex, const FString& FlowNodeDebugName /*= TEXT("")*/)
{
	return GetOrAddBranch(BranchIndex)->QueueFunction(FlowNodeDebugName);
}

FControlFlowWaitDelegate& FControlFlowTask_BranchLegacy::QueueWait(int32 BranchIndex, const FString& FlowNodeDebugName /*= TEXT("")*/)
{
	return GetOrAddBranch(BranchIndex)->QueueWait(FlowNodeDebugName);
}

FControlFlowPopulator& FControlFlowTask_BranchLegacy::QueueControlFlow(int32 BranchIndex, const FString& TaskName /*= TEXT("")*/, const FString& FlowNodeDebugName /*= TEXT("")*/)
{
	return GetOrAddBranch(BranchIndex)->QueueControlFlow(TaskName, FlowNodeDebugName);
}

TSharedRef<FControlFlowTask_BranchLegacy> FControlFlowTask_BranchLegacy::QueueBranch(int32 BranchIndex, FControlFlowBranchDecider_Legacy& BranchDecider, const FString& TaskName /*= TEXT("")*/, const FString& FlowNodeDebugName /*= TEXT("")*/)
{
	return GetOrAddBranch(BranchIndex)->QueueBranch(BranchDecider, TaskName, FlowNodeDebugName);
}

FControlFlowPopulator& FControlFlowTask_BranchLegacy::QueueLoop(int32 BranchIndex, FControlFlowLoopComplete& LoopCompleteDelgate, const FString& TaskName /*= TEXT("")*/, const FString& FlowNodeDebugName /*= TEXT("")*/)
{
	return GetOrAddBranch(BranchIndex)->QueueLoop(LoopCompleteDelgate, TaskName, FlowNodeDebugName);
}

void FControlFlowTask_BranchLegacy::HandleBranchCompleted()
{
	Branches.Reset();

	OnComplete().ExecuteIfBound();
}

void FControlFlowTask_BranchLegacy::HandleBranchCancelled()
{
	Branches.Reset();

	OnCancelled().ExecuteIfBound();
}

TSharedRef<FControlFlow> FControlFlowTask_BranchLegacy::GetOrAddBranch(int32 BranchIndex)
{
	if (!Branches.Contains(BranchIndex))
	{
		TSharedRef<FControlFlow> NewBranchFlow = MakeShared<FControlFlow>();

		NewBranchFlow->ParentFlow = GetOwningFlowForTaskNode();
		
		Branches.Add(BranchIndex, NewBranchFlow);
	}

	return Branches[BranchIndex];
}

void FControlFlowTask_BranchLegacy::Execute()
{
	if (BranchDelegate.IsBound())
	{
		SelectedBranch = BranchDelegate.Execute();

		TSharedRef<FControlFlow> FlowToExecute = GetOrAddBranch(*SelectedBranch);

		FlowToExecute->LastZeroSecondDelay = GetOwningFlowForTaskNode().Pin()->LastZeroSecondDelay;

		FlowToExecute->OnCompleteDelegate_Internal.BindSP(SharedThis(this), &FControlFlowTask_BranchLegacy::HandleBranchCompleted);
		FlowToExecute->OnExecutedWithoutAnyNodesDelegate_Internal.BindSP(SharedThis(this), &FControlFlowTask_BranchLegacy::HandleBranchCompleted);
		FlowToExecute->OnCancelledDelegate_Internal.BindSP(SharedThis(this), &FControlFlowTask_BranchLegacy::HandleBranchCancelled);

		FlowToExecute->ExecuteFlow();
	}
	else
	{
		HandleBranchCompleted();
	}
}

void FControlFlowTask_BranchLegacy::Cancel()
{
	if (ensureAlways(SelectedBranch.IsSet()) && Branches.Contains(*SelectedBranch) && Branches[*SelectedBranch]->IsRunning())
	{
		Branches[*SelectedBranch]->CancelFlow();
	}
	else
	{
		HandleBranchCancelled();
	}
}

//////////////////////////////////
//FControlFlowSimpleSubTask
//////////////////////////////////

FControlFlowSimpleSubTask::FControlFlowSimpleSubTask(const FString& TaskName)
	: FControlFlowSubTaskBase(TaskName)
	, TaskFlow(MakeShared<FControlFlow>(TaskName))
{
}

void FControlFlowSimpleSubTask::Execute()
{
	if (TaskPopulator.IsBound() && GetTaskFlow().IsValid())
	{
		GetTaskFlow()->LastZeroSecondDelay = GetOwningFlowForTaskNode().Pin()->LastZeroSecondDelay;

		GetTaskFlow()->OnCompleteDelegate_Internal.BindSP(SharedThis(this), &FControlFlowSimpleSubTask::CompletedSubTask);
		GetTaskFlow()->OnExecutedWithoutAnyNodesDelegate_Internal.BindSP(SharedThis(this), &FControlFlowSimpleSubTask::CompletedSubTask);
		GetTaskFlow()->OnCancelledDelegate_Internal.BindSP(SharedThis(this), &FControlFlowSimpleSubTask::CancelledSubTask);

		TaskPopulator.Execute(GetTaskFlow().ToSharedRef());

		ensureAlwaysMsgf(!GetTaskFlow()->IsRunning(), TEXT("Did you call ExecuteFlow() on a SubFlow? You don't need to."));

		GetTaskFlow()->ExecuteFlow();
	}
	else
	{
		UE_LOGF(LogControlFlows, Error, "ControlFlow - Executed Sub Task (%ls) without proper set up", *GetTaskName());

		CompletedSubTask();
	}
}

void FControlFlowSimpleSubTask::Cancel()
{
	if (GetTaskFlow().IsValid() && GetTaskFlow()->IsRunning())
	{
		GetTaskFlow()->CancelFlow();
	}
	else
	{
		CancelledSubTask();
	}
}

void FControlFlowSimpleSubTask::CompletedSubTask()
{
	OnComplete().ExecuteIfBound();
}

void FControlFlowSimpleSubTask::CancelledSubTask()
{
	OnCancelled().ExecuteIfBound();
}

//////////////////////////////////
//FControlFlowLoop
//////////////////////////////////

FControlFlowTask_LoopDeprecated::FControlFlowTask_LoopDeprecated(FControlFlowLoopComplete& TaskCompleteDelegate, const FString& TaskName)
	: FControlFlowSimpleSubTask(TaskName)
	, TaskCompleteDecider(TaskCompleteDelegate)
{}

void FControlFlowTask_LoopDeprecated::Execute()
{
	if (GetTaskPopulator().IsBound() && TaskCompleteDecider.IsBound() && GetTaskFlow().IsValid())
	{
		if (TaskCompleteDecider.Execute())
		{
			CompletedLoop();
		}
		else
		{
			GetTaskFlow()->LastZeroSecondDelay = GetOwningFlowForTaskNode().Pin()->LastZeroSecondDelay;

			GetTaskFlow()->OnCompleteDelegate_Internal.BindSP(SharedThis(this), &FControlFlowTask_LoopDeprecated::CompletedLoop);
			GetTaskFlow()->OnExecutedWithoutAnyNodesDelegate_Internal.BindSP(SharedThis(this), &FControlFlowTask_LoopDeprecated::CompletedLoop);
			GetTaskFlow()->OnCancelledDelegate_Internal.BindSP(SharedThis(this), &FControlFlowTask_LoopDeprecated::CancelledLoop);

			GetTaskPopulator().Execute(GetTaskFlow().ToSharedRef());

			GetTaskFlow()->ExecuteFlow();
		}
	}
	else
	{
		UE_LOGF(LogControlFlows, Error, "ControlFlow - Executed Loop (%ls) without proper bound delegates", *GetTaskName());

		CompletedLoop();
	}
}

void FControlFlowTask_LoopDeprecated::Cancel()
{
	if (GetTaskFlow().IsValid() && GetTaskFlow()->IsRunning())
	{
		GetTaskFlow()->CancelFlow();
	}
	else
	{
		CancelledLoop();
	}
}

void FControlFlowTask_LoopDeprecated::CompletedLoop()
{
	if (TaskCompleteDecider.IsBound() && !TaskCompleteDecider.Execute())
	{
		Execute();
	}
	else
	{
		OnComplete().ExecuteIfBound();
	}
}

void FControlFlowTask_LoopDeprecated::CancelledLoop()
{
	OnCancelled().ExecuteIfBound();
}

//////////////////////////////////
//FControlFlowTask_Branch
//////////////////////////////////

void FControlFlowTask_Branch::Execute()
{
	if (ensureAlways(BranchDelegate.IsBound() && !SelectedBranchFlow.IsValid()))
	{
		TSharedRef<FControlFlowBranch> BranchDefinitions = MakeShared<FControlFlowBranch>();
		int32 SelectedBranchKey = BranchDelegate.Execute(BranchDefinitions);

		if (ensureAlwaysMsgf(BranchDefinitions->Contains(SelectedBranchKey), TEXT("You've returned a Branch Key that doesn't exist!")))
		{
			SelectedBranchFlow = BranchDefinitions->FindChecked(SelectedBranchKey);

			SelectedBranchFlow->ParentFlow = GetOwningFlowForTaskNode();
			SelectedBranchFlow->LastZeroSecondDelay = GetOwningFlowForTaskNode().Pin()->LastZeroSecondDelay;

			SelectedBranchFlow->Activity = Activity;

			SelectedBranchFlow->OnCompleteDelegate_Internal.BindSP(SharedThis(this), &FControlFlowTask_Branch::HandleBranchCompleted);
			SelectedBranchFlow->OnExecutedWithoutAnyNodesDelegate_Internal.BindSP(SharedThis(this), &FControlFlowTask_Branch::HandleBranchCompleted);
			SelectedBranchFlow->OnCancelledDelegate_Internal.BindSP(SharedThis(this), &FControlFlowTask_Branch::HandleBranchCancelled);

			ensureAlwaysMsgf(!BranchDefinitions->IsAnyBranchRunning(), TEXT("Did you call ExecuteFlow() on a Branch? Do not do this! You only need to call ExecuteFlow once per FControlFlowStatics::Create!"));

			SelectedBranchFlow->ExecuteFlow();
		}
		else
		{
			HandleBranchCompleted();
		}
	}
	else
	{
		HandleBranchCompleted();
	}
}

void FControlFlowTask_Branch::Cancel()
{
	if (SelectedBranchFlow.IsValid() && ensureAlways(SelectedBranchFlow->IsRunning()))
	{
		SelectedBranchFlow->CancelFlow();
	}
	else
	{
		HandleBranchCancelled();
	}
}

void FControlFlowTask_Branch::HandleBranchCompleted()
{
	SelectedBranchFlow.Reset();

	if (BranchDelegate.IsBound())
	{
		BranchDelegate.Unbind();
	}

	OnComplete().ExecuteIfBound();
}

void FControlFlowTask_Branch::HandleBranchCancelled()
{
	SelectedBranchFlow.Reset();

	if (BranchDelegate.IsBound())
	{
		BranchDelegate.Unbind();
	}

	OnCancelled().ExecuteIfBound();
}

//////////////////////////////////
//FControlFlowTask_ConcurrentBranch
//////////////////////////////////

void FControlFlowTask_ConcurrentFlows::Execute()
{
	if (ensureAlways(ConcurrentFlowDelegate.IsBound() && !ConcurrentFlows.IsValid()))
	{
		ConcurrentFlows = MakeShared<FConcurrentControlFlows>();
		ConcurrentFlows->OwningTask = AsWeak();

		ConcurrentFlows->OnConcurrencyCompleted.BindSP(SharedThis(this), &FControlFlowTask_ConcurrentFlows::HandleConcurrentFlowsCompleted);
		ConcurrentFlows->OnConcurrencyCancelled.BindSP(SharedThis(this), &FControlFlowTask_ConcurrentFlows::HandleConcurrentFlowsCancelled);

		ConcurrentFlowDelegate.Execute(ConcurrentFlows.ToSharedRef());
		ConcurrentFlows->Execute();
	}
	else
	{
		HandleConcurrentFlowsCompleted();
	}
}

void FControlFlowTask_ConcurrentFlows::Cancel()
{
	if (ConcurrentFlows.IsValid())
	{
		ConcurrentFlows->CancelAll();
	}
	else
	{
		HandleConcurrentFlowsCancelled();
	}
}

void FControlFlowTask_ConcurrentFlows::HandleConcurrentFlowsCompleted()
{
	if (ConcurrentFlowDelegate.IsBound())
	{
		ConcurrentFlowDelegate.Unbind();
	}

	OnComplete().ExecuteIfBound();
}

void FControlFlowTask_ConcurrentFlows::HandleConcurrentFlowsCancelled()
{
	if (ConcurrentFlowDelegate.IsBound())
	{
		ConcurrentFlowDelegate.Unbind();
	}

	OnCancelled().ExecuteIfBound();
}
