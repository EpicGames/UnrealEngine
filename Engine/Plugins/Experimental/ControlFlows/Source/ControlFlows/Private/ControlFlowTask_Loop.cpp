// Copyright Epic Games, Inc. All Rights Reserved.

#include "ControlFlowTask_Loop.h"

#include "ControlFlows.h"
#include "ControlFlowConditionalLoop.h"

void FControlFlowTask_ConditionalLoop::Execute()
{
	if (!bStopLoopingOnUnboundedDelegate && ensureAlways(ConditionalLoopDelegate.IsBound() && !ConditionalLoop.IsValid()))
	{
		ConditionalLoop = MakeShared<FConditionalLoop>();

		ConditionalLoop->FlowLoop->ParentFlow = GetOwningFlowForTaskNode();

		ConditionalLoop->FlowLoop->OnCompleteDelegate_Internal.BindSP(SharedThis(this), &FControlFlowTask_ConditionalLoop::HandleLoopCompleted);
		ConditionalLoop->FlowLoop->OnExecutedWithoutAnyNodesDelegate_Internal.BindSP(SharedThis(this), &FControlFlowTask_ConditionalLoop::HandleLoopCompleted);
		ConditionalLoop->FlowLoop->OnCancelledDelegate_Internal.BindSP(SharedThis(this), &FControlFlowTask_ConditionalLoop::HandleLoopCancelled);
		ConditionalLoop->FlowLoop->OnNodeWasNotBoundedOnExecution_Internal.BindSP(SharedThis(this), &FControlFlowTask_ConditionalLoop::HandleOnNodeWasNotBoundedOnExecution);

		const EConditionalLoopResult InitialConditionalValue = ConditionalLoopDelegate.Execute(ConditionalLoop.ToSharedRef());

		if (ConditionalLoop->FlowLoop->IsRunning())
		{
			ensureAlwaysMsgf(false, TEXT("Did you call ExecuteFlow() on a Loop? Do not do this! You only need to call ExecuteFlow once per FControlFlowStatics::Create!"));
		}
		else
		{
			if (ensureAlwaysMsgf(ConditionalLoop->CheckConditionalFirst.IsSet(), TEXT("Unused Coonditional Loop Handle. Will skip this step.")))
			{
				bHandleUsed = true;
				if (ConditionalLoop->CheckConditionalFirst.GetValue())
				{
					Helper_ConditionalCheck(InitialConditionalValue);
				}
				else
				{
					ConditionalLoop->FlowLoop->LastZeroSecondDelay = GetOwningFlowForTaskNode().Pin()->LastZeroSecondDelay;
					ConditionalLoop->FlowLoop->ExecuteFlow();
				}
			}
			else
			{
				OnComplete().ExecuteIfBound();
			}
		}
	}
	else
	{
		OnComplete().ExecuteIfBound();
	}
}

void FControlFlowTask_ConditionalLoop::Cancel()
{
	if (ConditionalLoop.IsValid() && ConditionalLoop->FlowLoop->IsRunning())
	{
		ConditionalLoop->FlowLoop->CancelFlow();
	}
	else
	{
		HandleLoopCancelled();
	}
}
void FControlFlowTask_ConditionalLoop::Helper_ConditionalCheck(const EConditionalLoopResult InCondition) const
{
	if (InCondition == EConditionalLoopResult::RunLoop)
	{
		ConditionalLoop->FlowLoop->LastZeroSecondDelay = GetOwningFlowForTaskNode().Pin()->LastZeroSecondDelay;
		ConditionalLoop->FlowLoop->ExecuteFlow();
	}
	else if (InCondition == EConditionalLoopResult::LoopFinished)
	{
		OnComplete().ExecuteIfBound();
	}
	else
	{
		ensureAlwaysMsgf(false, TEXT("Unhandled Enum value"));
		OnComplete().ExecuteIfBound();
	}
}

void FControlFlowTask_ConditionalLoop::HandleLoopCompleted()
{
	ensureMsgf(bHandleUsed, TEXT("Did you call ExecuteFlow() on a Loop? Do not do this! This can cause a stack overflow!"));

	if (!bStopLoopingOnUnboundedDelegate && ensureAlways(ConditionalLoopDelegate.IsBound() && ConditionalLoop.IsValid()))
	{
		Helper_ConditionalCheck(ConditionalLoopDelegate.Execute(ConditionalLoop.ToSharedRef()));
	}
}

void FControlFlowTask_ConditionalLoop::HandleLoopCancelled()
{
	OnCancelled().ExecuteIfBound();
}

void FControlFlowTask_ConditionalLoop::HandleOnNodeWasNotBoundedOnExecution()
{
	UE_LOGF(LogControlFlows, Warning, "[Conditional Loop: %ls] %ls", ANSI_TO_TCHAR(__FUNCTION__), *GetTaskName());
	bStopLoopingOnUnboundedDelegate = true;
}
