// Copyright Epic Games, Inc. All Rights Reserved.

#include "ControlFlows.h"
#include "Modules/ModuleManager.h"

DEFINE_LOG_CATEGORY(LogControlFlows);

#define LOCTEXT_NAMESPACE "FControlFlowsModule"

void FControlFlowsModule::StartupModule()
{
	// This code will execute after your module is loaded into memory; the exact timing is specified in the .uplugin file per-module
}

void FControlFlowsModule::ShutdownModule()
{
	// This function may be called during shutdown to clean up your module.  For modules that support dynamic reloading,
	// we call this function before unloading the module.
}

#undef LOCTEXT_NAMESPACE
	
IMPLEMENT_MODULE(FControlFlowsModule, ControlFlows)
