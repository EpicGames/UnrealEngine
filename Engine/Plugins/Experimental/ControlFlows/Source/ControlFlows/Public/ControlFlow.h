// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "ControlFlowNode.h"
#include "Misc/OptionalFwd.h"

class UObject;
class FControlFlowTask_Branch;
class FControlFlowTask_BranchLegacy;

/**
 *  System/Tool to queue (asynchronous or synchronous) functions for modularity implemented via delegates.
 *  Allows code to be more easily read so there fewer 'Alt+G'-ing around to figure out what and where a class does it's thing
 *
 *  This system only supports non-const functions currently.
 *
 *  'QueueFunction': Queues a 'void (...)' function.
 *                   The Flow will execute this function and continue on with the next function in queue.
 *
 *  'QueueWait': Queues a 'void (FControlFlowNodeRef FlowHandle, ...)' function.
 *               The flow will stop until 'FlowHandle->ContinueFlow()' is called.
 *	             BE RESPONSIBLE and make sure all code paths call to continue xOR cancel, otherwise the flow will hang.
 *
 *  'QueueControlFlow': Queues a 'void (TSharedRef<FControlFlow> Subflow, ...)' function.
 * 
 *  'QueueControlFlowBranch': Queues a 'int32(TSharedRef<FControlFlowBranch> Branch, ...)' function.
 *							  TODO: Allow any return-type for branch selection, and not restricted to only int32.
 *							  Lambda-Syntax: '.BranchFlow([this](TSharedRef<FControlFlowBranch> Branch) { })'
 * 
 *  'QueueConcurrentFlows': Queues a 'void(TSharedRef<FConcurrentControlFlows> ForkedFlow, ...)' function
 *							Lambda-Syntax: '.ForkFlow([this](TSharedRef<FConcurrentControlFlows> Fork) { })'
 * 
 *  'QueueConditionalLoop': Queues a 'EConditionalLoopResult(TSharedRef<FConditionalLoop> Loop, ...)' function
 *							The public functions of 'FConditionalLoop' forces the caller to define the loop as as a 'while (CONDITION) {}' or a 'do {} while(CONDITION);'
 *							The return is a FControlFlow& and queue your flow as normal
 *							Lambda-Syntax: '.Loop([this](TSharedRef<FConditionalLoop> Loop) { })'
 * 
 *  'QueueStep': Usable in #UObject's or classes that derive from #TSharedFromThis<UserClass>. The Control Flow will automatically deduce if this is a
 *				 '#QueueFunction', '#QueueWait', '#QueueControlFlow', '#QueueControlFlowBranch', '#QueueConcurrentFlows', '#EConditionalLoopResult' based on the function signature.
 *				 Returns a ref to the ControlFlow - enables chained step queueing.
 *
 *  Using the auto-deduction of 'QueueStep', you can change the queue from a synchronous function (QueueFunction) to an asynchronous one (QueueWait) or vice-versa
 *  by adding/removing the 'FControlFlowNodeRef FlowHandle' as your first parameter. And you can change it to (QueueControlFlow) if need be as well!
 * 
 *  Syntax:
 * 
 *  void MyFunction(...);
 *  void MyFunction(FControlFlowNodeRef FlowHandle, ...);
 *  void MyFunction(TSharedRef<FControlFlow> Subflow, ...);
 *  void MyFunction(int32(TSharedRef<FControlFlowBranch> Branch, ...);
 * 
 *  ControlFlowInstance->QueueStep(this, &UserClass:MyFunction1, ...)
 *		.QueueStep(this, &UserClass:MyFunction2, ...)
 *		.QueueStep(this, &UserClass:MyFunction3, ...);
 * 
 *  This allow ease of going from Synchronous Functionality to Asynchronously Functionality to Subflows as you build out your Flow.
 * 
 *  ******************
 *  Full Example Class
 * 
 *	struct FMyFlowClass : public TSharedFromThis<FMyFlowClass>
 *	{
 *		typedef FMyFlowClass ThisClass;
 * 
 * 		FMyFlowClass() : MyPurpose(MakeShared<FControlFlow>()) {}
 *
 * 		void RunMyPurpose()
 * 		{
 * 			MyPurpose
 *				.QueueStep(this, &ThisClass::Construct)
 *				.Loop([this](TSharedRef<FConditionalLoop> Outerloop)
 *				{
 *					Outerloop->RunLoopFirst()
 *						.SetCancelledNodeAsComplete(true)
 *						.QueueStep(this, &ThisClass::Foo)
 *						.QueueStep(this, &ThisClass::Foo)
 *						.Loop([this](TSharedRef<FConditionalLoop> InnerLoop)
 *						{
 *							InnerLoop->CheckConditionFirst()
 *								.BranchFlow([this], TSharedRef<FConditionalLoop> Branch)
 *								{
 *									Branch->AddOrGetBranch(0)
 *										.QueueStep(this, &ThisClass::Foo)
 *										.QueueStep(this, &ThisClass::Foo);
 * 
 *									Branch->AddOrGetBranch(1)
 *										.QueueStep(this, &ThisClass::Foo)
 *										.ForkFlow([this], TSharedRef<FConcurrentControlFlows> ConcurrentFlows)
 *										{
 *											ConcurrentFlows->AddOrGetFlow(0)
 *												.QueueStep(this, &ThisClass::Foo)
 *												.QueueStep(this, &ThisClass::Foo);
 * 
 *											ConcurrentFlows->AddOrGetFlow(1)
 *												.QueueStep(this, &ThisClass::Foo);
 *										});
 * 
 *									return FMath::RandBool() ? EConditionalLoopResult::RunLoop : EConditionalLoopResult::LoopFinished;
 *								})
 *								.QueueStep(this, &ThisClass::Foo);
 *
 *							return FMath::RandBool() ? EConditionalLoopResult::RunLoop : EConditionalLoopResult::LoopFinished;
 *						})
 *						.QueueStep(this, &ThisClass::Foo);
 *
 *					return EConditionalLoopResult::RunLoop; // OuterLoop will never end in this example
 *				})
 *				.QueueStep(this, &ThisClass::Destruct)
 *				.ExecuteFlow();
 *		}
 *
 *	private:
 *		void Foo();
 *		void Construct(); // Equivalent to Init
 *		void Destruct(); // Equivalent to Uninit
 *
 *		TSharedRef<FControlFlow> MyPurpose;
 *	};
 * 
 * The implementation details of a flow can be fully disjointed from the Flow logic itself!
 */

class FControlFlow : public TSharedFromThis<FControlFlow>
{
public:
	CONTROLFLOWS_API FControlFlow(const FString& FlowDebugName = TEXT(""));

public:
	/** This needs to be called, otherwise nothing will happen!! Call after you finish adding functions to the queue. Calling with an empty queue is safe. */
	CONTROLFLOWS_API void ExecuteFlow();
	CONTROLFLOWS_API void Reset();
	CONTROLFLOWS_API bool IsRunning() const;
	size_t NumInQueue() const { return FlowQueue.Num(); }

	FSimpleMulticastDelegate& OnNodeComplete() const { return OnStepCompletedDelegate; }
	FSimpleMulticastDelegate& OnFlowComplete() const { return OnFlowCompleteDelegate; }
	FSimpleMulticastDelegate& OnFlowCancel() const { return OnFlowCancelledDelegate; }

	/** Will cancel ALL flows, both child ControlFlows and ControlFlows who owns this Flow. You've been warned. */
	CONTROLFLOWS_API void CancelFlow();

	CONTROLFLOWS_API FControlFlow& SetCancelledNodeAsComplete(bool bCancelledNodeIsComplete);

	CONTROLFLOWS_API TOptional<FString> GetCurrentStepDebugName() const;

	CONTROLFLOWS_API TSharedPtr<FTrackedActivity> GetTrackedActivity() const;

public:
	CONTROLFLOWS_API FControlFlow& QueueDelay(const float InSeconds, const FString& NodeName = FString());
	CONTROLFLOWS_API FControlFlow& QueueSetCancelledNodeAsComplete(const bool bCancelledNodeIsComplete, const FString& NodeName = FString());

	template<typename...ArgsT>
	FControlFlow& QueueStep(const FString& NodeName, ArgsT...Params)
	{
		QueueStep_Internal(FormatOrGetNewNodeDebugName(NodeName), Params...);
		return *this;
	}

	template<typename...ArgsT>
	FControlFlow& QueueStep(const TCHAR* NodeName, ArgsT...Params)
	{
		QueueStep_Internal(FormatOrGetNewNodeDebugName(NodeName), Params...);
		return *this;
	}

	template<typename...ArgsT>
	FControlFlow& QueueStep(const char* NodeName, ArgsT...Params)
	{
		QueueStep_Internal(FormatOrGetNewNodeDebugName(NodeName), Params...);
		return *this;
	}

	template<typename...ArgsT>
	FControlFlow& QueueStep(ArgsT...Params)
	{
		QueueStep_Internal(FormatOrGetNewNodeDebugName(), Params...);
		return *this;
	}

public:
	template<typename FunctionT, typename...ArgsT>
	FControlFlow& Loop(FunctionT InLoopLambda, ArgsT...Params)
	{
		QueueConditionalLoop(FormatOrGetNewNodeDebugName()).BindLambda(InLoopLambda, Params...);
		return *this;
	}

	template<typename...ArgsT>
	FControlFlow& SwitchFlow(ArgsT...Args)
	{
		return BranchFlow(Args...);
	}

	template<typename FunctionT, typename...ArgsT>
	FControlFlow& BranchFlow(FunctionT InBranchLambda, ArgsT...Params)
	{
		QueueControlFlowBranch(FormatOrGetNewNodeDebugName()).BindLambda(InBranchLambda, Params...);
		return *this;
	}

	template<typename...ArgsT>
	FControlFlow& SplitFlow(ArgsT...Args)
	{
		return ForkFlow(Args...);
	}

	template<typename FunctionT, typename...ArgsT>
	FControlFlow& ForkFlow(FunctionT InForkLambda, ArgsT...Params)
	{
		QueueConcurrentFlows(FormatOrGetNewNodeDebugName()).BindLambda(InForkLambda, Params...);
		return *this;
	}

	CONTROLFLOWS_API FControlFlow& TrackActivities(TSharedPtr<FTrackedActivity> InActivity = nullptr);

public:
	CONTROLFLOWS_API FSimpleDelegate& QueueFunction(const FString& FlowNodeDebugName = TEXT(""));
	CONTROLFLOWS_API FControlFlowWaitDelegate& QueueWait(const FString& FlowNodeDebugName = TEXT(""));
	CONTROLFLOWS_API FControlFlowPopulator& QueueControlFlow(const FString& TaskName = TEXT(""), const FString& FlowNodeDebugName = TEXT(""));
	CONTROLFLOWS_API FControlFlowBranchDefiner& QueueControlFlowBranch(const FString& TaskName = TEXT(""), const FString& FlowNodeDebugName = TEXT(""));
	CONTROLFLOWS_API FConcurrentFlowsDefiner& QueueConcurrentFlows(const FString& TaskName = TEXT(""), const FString& FlowNodeDebugName = TEXT(""));
	CONTROLFLOWS_API FControlFlowConditionalLoopDefiner& QueueConditionalLoop(const FString& TaskName = TEXT(""), const FString& FlowNodeDebugName = TEXT(""));

private:
	template<typename BindingObjectT, typename...PayloadParamsT>
	void QueueStep_Internal(const FString& InDebugName, TSharedRef<BindingObjectT> InBindingObject, PayloadParamsT...Params)
	{
		//ensureMsgf(false, TEXT("Deprecated. Pass the raw pointer of the TSharedFromThis Binding Object (e.g. 'this') into QueueStep."));

		QueueStep_Internal_TSharedFromThis(InDebugName, InBindingObject, Params...);
	}

	template<typename BindingObjectT, typename...ArgsT>
	void QueueStep_Internal(const FString& InDebugName, BindingObjectT* InBindingObject, ArgsT...Params)
	{
		QueueStep_Internal_DeduceBindingObject<BindingObjectT>(InDebugName, InBindingObject, Params...);
	}

private:
	template<typename BindingObjectClassT, typename...ArgsT>
	void QueueStep_Internal_DeduceBindingObject(const FString& InDebugName, typename TEnableIf<IsDerivedFromSharedFromThis<BindingObjectClassT>(), BindingObjectClassT*>::Type InBindingObject, ArgsT...Params)
	{
		if (InBindingObject->DoesSharedInstanceExist())
		{
			QueueStep_Internal_TSharedFromThis(InDebugName, StaticCastSharedRef<BindingObjectClassT>(InBindingObject->AsShared()), Params...);
		}
	}

	template<typename BindingObjectClassT, typename...ArgsT>
	void QueueStep_Internal_DeduceBindingObject(const FString& InDebugName, typename TEnableIf<TIsDerivedFrom<BindingObjectClassT, UObject>::IsDerived, BindingObjectClassT*>::Type InBindingObject, ArgsT...Params)
	{
		QueueStep_Internal_UObject<BindingObjectClassT>(InDebugName, InBindingObject, Params...);
	}

private:
	
	template<typename BindingObjectClassT, typename...PayloadParamsT>
	void QueueStep_Internal_TSharedFromThis(const FString& InDebugName, TSharedRef<BindingObjectClassT> InBindingObject, typename TMemFunPtrType<false, BindingObjectClassT, EConditionalLoopResult(TSharedRef<FConditionalLoop>, PayloadParamsT...)>::Type InFunction, PayloadParamsT...Params)
	{
		QueueConditionalLoop(InDebugName).BindSP(InBindingObject, InFunction, Params...);
	}

	template<typename BindingObjectClassT, typename...PayloadParamsT>
	void QueueStep_Internal_TSharedFromThis(const FString& InDebugName, TSharedRef<BindingObjectClassT> InBindingObject, typename TMemFunPtrType<false, BindingObjectClassT, void(TSharedRef<FConcurrentControlFlows>, PayloadParamsT...)>::Type InFunction, PayloadParamsT...Params)
	{
		QueueConcurrentFlows(InDebugName).BindSP(InBindingObject, InFunction, Params...);
	}

	template<typename BindingObjectClassT, typename...PayloadParamsT>
	void QueueStep_Internal_TSharedFromThis(const FString& InDebugName, TSharedRef<BindingObjectClassT> InBindingObject, typename TMemFunPtrType<false, BindingObjectClassT, int32(TSharedRef<FControlFlowBranch>, PayloadParamsT...)>::Type InFunction, PayloadParamsT...Params)
	{
		QueueControlFlowBranch(InDebugName).BindSP(InBindingObject, InFunction, Params...);
	}

	template<typename BindingObjectClassT, typename...PayloadParamsT>
	void QueueStep_Internal_TSharedFromThis(const FString& InDebugName, TSharedRef<BindingObjectClassT> InBindingObject, typename TMemFunPtrType<false, BindingObjectClassT, void(FControlFlowNodeRef, PayloadParamsT...)>::Type InFunction, PayloadParamsT...Params)
	{
		QueueWait(InDebugName).BindSP(InBindingObject, InFunction, Params...);
	}

	template<typename BindingObjectClassT, typename...PayloadParamsT>
	void QueueStep_Internal_TSharedFromThis(const FString& InDebugName, TSharedRef<BindingObjectClassT> InBindingObject, typename TMemFunPtrType<false, BindingObjectClassT, void(TSharedRef<FControlFlow>, PayloadParamsT...)>::Type InFunction, PayloadParamsT...Params)
	{
		QueueControlFlow(InDebugName).BindSP(InBindingObject, InFunction, Params...);
	}

	template<typename BindingObjectClassT, typename...PayloadParamsT>
	void QueueStep_Internal_TSharedFromThis(const FString& InDebugName, TSharedRef<BindingObjectClassT> InBindingObject, typename TMemFunPtrType<false, BindingObjectClassT, void(PayloadParamsT...)>::Type InFunction, PayloadParamsT...Params)
	{
		QueueFunction(InDebugName).BindSP(InBindingObject, InFunction, Params...);
	}

private:

	template<typename BindingObjectClassT, typename...PayloadParamsT>
	void QueueStep_Internal_UObject(const FString& InDebugName, BindingObjectClassT* InBindingObject, typename TMemFunPtrType<false, BindingObjectClassT, EConditionalLoopResult(TSharedRef<FConditionalLoop>, PayloadParamsT...)>::Type InFunction, PayloadParamsT...Params)
	{
		QueueConditionalLoop(InDebugName).BindUObject(InBindingObject, InFunction, Params...);
	}

	template<typename BindingObjectClassT, typename...PayloadParamsT>
	void QueueStep_Internal_UObject(const FString& InDebugName, BindingObjectClassT* InBindingObject, typename TMemFunPtrType<false, BindingObjectClassT, void(TSharedRef<FConcurrentControlFlows>, PayloadParamsT...)>::Type InFunction, PayloadParamsT...Params)
	{
		QueueConcurrentFlows(InDebugName).BindUObject(InBindingObject, InFunction, Params...);
	}

	template<typename BindingObjectClassT, typename...PayloadParamsT>
	void QueueStep_Internal_UObject(const FString& InDebugName, BindingObjectClassT* InBindingObject, typename TMemFunPtrType<false, BindingObjectClassT, int32(TSharedRef<FControlFlowBranch>, PayloadParamsT...)>::Type InFunction, PayloadParamsT...Params)
	{
		QueueControlFlowBranch(InDebugName).BindUObject(InBindingObject, InFunction, Params...);
	}

	template<typename BindingObjectClassT, typename...PayloadParamsT>
	void QueueStep_Internal_UObject(const FString& InDebugName, BindingObjectClassT* InBindingObject, typename TMemFunPtrType<false, BindingObjectClassT, void(FControlFlowNodeRef, PayloadParamsT...)>::Type InFunction, PayloadParamsT...Params)
	{
		QueueWait(InDebugName).BindUObject(InBindingObject, InFunction, Params...);
	}

	template<typename BindingObjectClassT, typename...PayloadParamsT>
	void QueueStep_Internal_UObject(const FString& InDebugName, BindingObjectClassT* InBindingObject, typename TMemFunPtrType<false, BindingObjectClassT, void(TSharedRef<FControlFlow>, PayloadParamsT...)>::Type InFunction, PayloadParamsT...Params)
	{
		QueueControlFlow(InDebugName).BindUObject(InBindingObject, InFunction, Params...);
	}

	template<typename BindingObjectClassT, typename...PayloadParamsT>
	void QueueStep_Internal_UObject(const FString& InDebugName, BindingObjectClassT* InBindingObject, typename TMemFunPtrType<false, BindingObjectClassT, void(PayloadParamsT...)>::Type InFunction, PayloadParamsT...Params)
	{
		QueueFunction(InDebugName).BindUObject(InBindingObject, InFunction, Params...);
	}

private:
	friend class FControlFlowNode;
	friend class FControlFlowNode_RequiresCallback;
	friend class FControlFlowNode_SelfCompleting;
	friend class FControlFlowSimpleSubTask;
	friend class FControlFlowTask_LoopDeprecated;
	friend class FControlFlowTask_BranchLegacy;
	friend class FControlFlowTask_Branch;
	friend class FControlFlowTask_ConditionalLoop;
	friend class FControlFlowStatics;
	friend class FConcurrentControlFlows;
	friend struct FConcurrencySubFlowContainer;

public:
	// Both of these implementations are deprecated
	CONTROLFLOWS_API TSharedRef<FControlFlowTask_BranchLegacy> QueueBranch(FControlFlowBranchDecider_Legacy& BranchDecider, const FString& TaskName = TEXT(""), const FString& FlowNodeDebugName = TEXT(""));
	CONTROLFLOWS_API FControlFlowPopulator& QueueLoop(FControlFlowLoopComplete& LoopCompleteDelgate, const FString& TaskName = TEXT(""), const FString& FlowNodeDebugName = TEXT(""));
	
private:
	inline void SetProfilerEventStarted() { ensureMsgf(!bProfilerEventStarted, TEXT("Started a new control flow profiler event before previous step completed.")); bProfilerEventStarted = true; }

	void HandleControlFlowNodeCompleted(const TSharedRef<const FControlFlowNode>& NodeCompleted);

	mutable FSimpleMulticastDelegate OnStepCompletedDelegate;
	mutable FSimpleMulticastDelegate OnFlowCompleteDelegate;
	mutable FSimpleMulticastDelegate OnFlowCancelledDelegate;
	
	mutable FSimpleDelegate OnCompleteDelegate_Internal;
	mutable FSimpleDelegate OnExecutedWithoutAnyNodesDelegate_Internal;
	mutable FSimpleDelegate OnCancelledDelegate_Internal;
	mutable FSimpleDelegate OnNodeWasNotBoundedOnExecution_Internal;

private:
	void ExecuteNextNodeInQueue();

	void ExecuteNode(TSharedRef<FControlFlowNode_SelfCompleting> SelfCompletingNode);
private:
	void HandleTaskNodeExecuted(TSharedRef<FControlFlowNode_Task> TaskNode);
	void HandleTaskNodeCancelled(TSharedRef<FControlFlowNode_Task> TaskNode);

	void HandleOnTaskComplete();
	void HandleOnTaskCancelled();

	void BroadcastCancellation();
private:

	void LogNodeExecution(const FControlFlowNode& NodeExecuted);

	FString GetFlowPath() const;

	int32 GetRepeatedFlowCount() const;

public:
	const FString& GetDebugName() const { return DebugName; }

private:
	CONTROLFLOWS_API FString FormatOrGetNewNodeDebugName(const FString& FlowNodeDebugName = TEXT(""));

private:

	static int32 UnnamedControlFlowCounter;

	FString DebugName;

	bool bInterpretCancelledNodeAsComplete = false;

	bool bBroadcastingStepComplete = false;

	int32 UnnamedNodeCounter = 0;

	int32 UnnamedBranchCounter = 0;

	TSharedPtr<FControlFlowNode_Task> CurrentlyRunningTask = nullptr;

	TSharedPtr<FControlFlowNode> CurrentNode = nullptr;

	TWeakPtr<FControlFlow> ParentFlow;

	TPair<double /*Timestamp*/, float /*DeltaTime*/> LastZeroSecondDelay;

	//TODO: Put behind some args, because this is expensive.
	TArray<TSharedRef<FControlFlow>> SubFlowStack_ForDebugging;

	TArray<TSharedRef<FControlFlowNode>> FlowQueue;

	TSharedPtr<FTrackedActivity> Activity;

	bool bProfilerEventStarted = false;
};

#if CPUPROFILERTRACE_ENABLED

#define CONTROL_FLOW_PERF_TRACE_STEP(FlowHandle, EventName) \
	TRACE_CPUPROFILER_EVENT_MANUAL_START(#EventName); \
	if (TRACE_CPUPROFILER_EVENT_MANUAL_IS_ENABLED()) \
	{ \
		FlowHandle->SetProfilerEventStarted(); \
	}

#else

#define CONTROL_FLOW_PERF_TRACE_STEP(FlowHandle, EventName)

#endif
