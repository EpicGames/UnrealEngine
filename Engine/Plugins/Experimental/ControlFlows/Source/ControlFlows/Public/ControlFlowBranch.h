// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "ControlFlow.h"

static int32 UnnamedControlFlowBranchCounter = 0;

class FControlFlowBranch : public TSharedFromThis<FControlFlowBranch>
{
public:
	FControlFlow& AddOrGetBranch(const int32& InKey, const FString& DebugBranchName = TEXT(""))
	{
		if (TSharedRef<FControlFlow>* ExistingBranch = Branches.Find(InKey))
		{
			return ExistingBranch->Get();
		}
		else
		{
			const FString& DebugNameToUse = DebugBranchName.IsEmpty() ? FString::Format(TEXT("UnnamedBranch_{0}"), { (UnnamedControlFlowBranchCounter++) }) : DebugBranchName;
			TSharedRef<FControlFlow> NewControlFlow = MakeShared<FControlFlow>(DebugNameToUse);

			Branches.Add(InKey, NewControlFlow);
			return NewControlFlow.Get();
		}
	}

	bool Contains(const int32& Key) const { return Branches.Contains(Key); }
	TSharedRef<FControlFlow>& FindChecked(const int32& Key) { return Branches.FindChecked(Key); }

	bool IsAnyBranchRunning() const
	{
		for (TPair<int32, TSharedRef<FControlFlow>> BranchPair : Branches)
		{
			if (BranchPair.Value->IsRunning())
			{
				return true;
			}
		}

		return false;
	}

private:
	TMap<int32, TSharedRef<FControlFlow>> Branches;
};
