// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Delegates/Delegate.h"

class FControlFlow;
class FControlFlowSubTaskBase;
class FConcurrentControlFlowBehavior;
struct FConcurrencySubFlowContainer;

enum class EConcurrentExecution
{
	Default, // Flows will execute in a single thread; always in the same order
	Random, // Flows will execute in a single thread; random order
	Parallel, // Flows will execute in actual separate threads
};

// All Flows will be executed concurrently (or in-parallel, See: EConcurrentExecution). WARNING: Having a non-terminating loop within a fork can cause a forever hang for the forked step
class FConcurrentControlFlows : public TSharedFromThis<FConcurrentControlFlows>
{
public:
	struct FParallelFlowLock {};

	CONTROLFLOWS_API FControlFlow& AddOrGetProng(int32 InIdentifier, const FString& DebugSubFlowName = TEXT(""));
	CONTROLFLOWS_API FControlFlow& AddOrGetFlow(int32 InIdentifier, const FString& DebugSubFlowName = TEXT(""));

	CONTROLFLOWS_API FConcurrentControlFlows& SetExecution(const EConcurrentExecution InBehavior);

	// TODO: Multi-threaded-like behavior
	// void Set(const FConcurrentControlFlowBehavior& InBehavior);

private:
	friend class FControlFlowTask_ConcurrentFlows;

	bool AreAllSubFlowsCompletedOrCancelled() const;
	bool HasAnySubFlowBeenExecuted() const;

	void HandleConcurrentFlowDone(int32 FlowIndex, const bool bCompleted);

	void CheckToBroadcastComplete();

	void Execute();
	void CancelAll(); // Do not make public

	void OnAllCompleted();
	void OnAllCancelled();

	FSimpleDelegate OnConcurrencyCompleted;
	FSimpleDelegate OnConcurrencyCancelled;

	EConcurrentExecution ExecutionBehavior = EConcurrentExecution::Default;
	bool bCancelAllHasBegun = false;
	TSharedPtr<FConcurrentControlFlows::FParallelFlowLock, ESPMode::ThreadSafe> ParallelFlowLock;
	TMap<int32, TSharedRef<FConcurrencySubFlowContainer>> ConcurrentFlows;

	TWeakPtr<FControlFlowSubTaskBase> OwningTask;

private:
	FConcurrentControlFlowBehavior GetConcurrencyBehavior() const;
};

struct FConcurrencySubFlowContainer : public TSharedFromThis<FConcurrencySubFlowContainer>
{
	FConcurrencySubFlowContainer(const FString& InDebugName);

private:
	friend class FConcurrentControlFlows;

	bool HasBeenExecuted() const;
	bool HasBeenCancelled() const;
	bool IsCompleteOrCancelled() const;

	void Execute(TWeakPtr<FConcurrentControlFlows::FParallelFlowLock> InFlowLock);
	void Cancel();
	FSimpleDelegate& OnComplete();
	FSimpleDelegate& OnExecutedWithoutAnyNodes();
	FSimpleDelegate& OnCancelled();
	const FString& GetDebugName();

	TSharedRef<FControlFlow> GetControlFlow() const;

	bool bHasBeenExecuted = false;
	bool bHasBeenCancelled = false;

	TSharedRef<FControlFlow> SubFlow;
	TSharedPtr<FConcurrentControlFlows::FParallelFlowLock, ESPMode::ThreadSafe> ParallelFlowLock;
};

// Placeholder class to extend Concurrency behavior
class FConcurrentControlFlowBehavior
{
	friend class FConcurrentControlFlows;
private:
	enum class EContinueConditions
	{
		Default,
		// TODO:
		// Default: Equivalent to "Sync" in verse: Outer flow will continue once all flows have completed or been cancelled
		//		See: "Default_Note"
		// 
		// Race: Outer flow will continue once the first flow completes or cancelled. Other flows cancelled
		//		Race_Complete: Outer flow will continue once the first flow complete (ignoring cancel). Other flows cancelled
		//		Race_Cancel: Outer flow will continue once the first flow cancel (ignoring complete). Other flows cancelled
		// 
		// Rush: Outer flow will continue once the first flow completes or cancelled. Other flows continue
		//		Rush_Complete: Outer flow will continue once the first flow complete (ignoring cancel). Other flows continue
		//		Rush_Cancel: Outer flow will continue once the first flow cancel (ignoring complete). Other flows continue
		// 
		// Default_Note: "Sync_Complete" and "Sync_Cancel" do not make sense and is unnecessary. If a flow is currently running, we have to wait for that flow to complete or cancel in order for the "Sync" condition to be satisfied.
		// 
		// Will allow to specify specific flows as an exception to Race/Rush
	};

	EContinueConditions GetContinueCondition() const { return EContinueConditions::Default; }
};
