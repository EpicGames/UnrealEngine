// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Templates/SharedPointer.h"
#include "Misc/Optional.h"

class FControlFlow;

// In contrast to, for example, iterator loops
class FConditionalLoop : public TSharedFromThis<FConditionalLoop>
{
public:
	CONTROLFLOWS_API FConditionalLoop();
	
	CONTROLFLOWS_API FControlFlow& CheckConditionFirst();
	CONTROLFLOWS_API FControlFlow& RunLoopFirst();
	CONTROLFLOWS_API FControlFlow& ExecuteAtLeastOnce();
	CONTROLFLOWS_API FControlFlow& SetCheckConditionFirst(bool bInValue);

private:
	friend class FControlFlowTask_ConditionalLoop;

	TOptional<bool> CheckConditionalFirst;
	TSharedRef<FControlFlow> FlowLoop;
};