// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

// HEADER_UNIT_SKIP - Not included directly

#include "Templates/SharedPointer.h"
#include "Templates/PointerVariants.h"

class FControlFlowContainerBase;
class FControlFlow;

namespace UE::Private
{
	static bool OwningObjectIsValid(TSharedRef<const FControlFlowContainerBase> InFlowContainer);
}

/* These wrapper classes along with FControlFlowStatics should be restructured in a way so that we do not rely on heap memory anymore
*  Specifically we do not derive any FControlFlow classes from `TSharedFromThis` nor `UObject`
*/

class FControlFlowContainerBase : public TSharedFromThis<FControlFlowContainerBase>
{
public:
	FControlFlowContainerBase() = delete;
	virtual ~FControlFlowContainerBase() = default;

	FControlFlowContainerBase(TSharedRef<FControlFlow> InFlow, const FString& FlowId)
		: ControlFlow(InFlow), FlowName(FlowId)
	{
		checkf(FlowName.Len() > 0, TEXT("All Flows need a non-empty ID!"));
	}

public:
	virtual bool OwningObjectEqualTo(const void* InObject) const = 0;

	const FString& GetFlowName() const { return FlowName; }

	TSharedRef<FControlFlow> GetControlFlow() const { return ControlFlow; }

private:
	virtual bool OwningObjectIsValid() const = 0;
	friend bool UE::Private::OwningObjectIsValid(TSharedRef<const FControlFlowContainerBase> InFlowContainer);

private:
	TSharedRef<FControlFlow> ControlFlow;

	FString FlowName;
};

template<typename T>
class TControlFlowContainer : public FControlFlowContainerBase
{
public:
	TControlFlowContainer() = delete;
	TControlFlowContainer(T* InOwner, TSharedRef<FControlFlow> InFlow, const FString& FlowId)
		: FControlFlowContainerBase(InFlow, FlowId)
		, OwningObject(InOwner)
	{}

private:
	virtual bool OwningObjectEqualTo(const void* InObject) const override final
	{
		return InObject != nullptr && OwningObject.IsValid() && OwningObject.Pin().Get() == InObject;
	}

	virtual bool OwningObjectIsValid() const override final
	{
		return OwningObject.IsValid();
	}

private:
	TWeakPtrVariant<T> OwningObject;
};

namespace UE::Private
{
	static inline bool OwningObjectIsValid(TSharedRef<const FControlFlowContainerBase> InFlowContainer)
	{
		return InFlowContainer->OwningObjectIsValid();
	}
}
