// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

// IWYU pragma: begin_keep
#include "ControlFlow.h"
#include "UObject/StrongObjectPtr.h"
#include "Containers/Ticker.h"
#include "ControlFlowContainer.h"
// IWYU pragma: end_keep

class FControlFlowStatics
{
public:
	template<typename OwningObjectT>
	static FControlFlow& Create(OwningObjectT* OwningObject, const FString& FlowId)
	{
		ensureMsgf(FlowId.Len() > 0, TEXT("All Flows need a non-empty ID!"));

		FControlFlow* AlreadyExistedFlow = Find(OwningObject, FlowId);
		if (!ensureMsgf(!AlreadyExistedFlow, TEXT("Flow already exists! All flows should have a unique ID. If there are multiple instances of the owning object, that might cause this! You can call MakeShared<FControlFlow> instead of using FControlFlowStatics!")))
		{
			AlreadyExistedFlow->Reset();
		}

		return AlreadyExistedFlow ? *AlreadyExistedFlow : Create_Internal(OwningObject, FlowId, false);
	}

public:
	template<typename OwningObjectT>
	static void StopFlow(OwningObjectT* OwningObject, const FString& FlowId)
	{
		FControlFlow* FlowToStop = Find(OwningObject, FlowId);
		if (ensureMsgf(FlowToStop && FlowToStop->IsRunning(), TEXT("Called to stop flow when it doesn't exist or not running!")))
		{
			FlowToStop->CancelFlow();

			CheckForInvalidFlows();
		}
	}

public:
	template<typename OwningObjectT>
	static FControlFlow* Find(OwningObjectT* OwningObject, const FString& FlowId)
	{
		TSharedPtr<FControlFlow> FoundFlow = Find_Internal(OwningObject, FlowId, GetNewlyCreatedFlows());
		if (!FoundFlow.IsValid()) { FoundFlow = Find_Internal(OwningObject, FlowId, GetExecutingFlows()); }
		if (!FoundFlow.IsValid()) { FoundFlow = Find_Internal(OwningObject, FlowId, GetFinishedFlows()); }
		if (!FoundFlow.IsValid()) { FoundFlow = Find_Internal(OwningObject, FlowId, GetPersistentFlows()); }

		return FoundFlow.Get();
	}

public:
	template<typename OwningObjectT>
	static FControlFlow& FindOrCreate(OwningObjectT* OwningObject, const FString& FlowId, bool bResetIfFound = false)
	{
		FControlFlow* AlreadyExistedFlow = Find(OwningObject, FlowId);
		if (AlreadyExistedFlow && bResetIfFound)
		{
			AlreadyExistedFlow->Reset();
		}

		return AlreadyExistedFlow ? *AlreadyExistedFlow : Create_Internal(OwningObject, FlowId, true);
	}

public:
	template<typename OwningObjectT>
	static bool IsRunning(OwningObjectT* OwningObject, const FString& FlowId)
	{
		FControlFlow* FoundFlow = Find(OwningObject, FlowId);
		return FoundFlow && FoundFlow->IsRunning();
	}

private:
	// These flows will be checked the next frame to make sure they are executed. If they are not, it will execute the flow and remove from this array.
	CONTROLFLOWS_API static TArray<TSharedRef<FControlFlowContainerBase>>& GetNewlyCreatedFlows();

	// These flows will not be checked the next frame, and will be moved to executing when we find ourselves executing.
	CONTROLFLOWS_API static TArray<TSharedRef<FControlFlowContainerBase>>& GetPersistentFlows();

	// Flows that are actively running.
	CONTROLFLOWS_API static TArray<TSharedRef<FControlFlowContainerBase>>& GetExecutingFlows();

	// Final Array that flows are moved to before being deleted.
	CONTROLFLOWS_API static TArray<TSharedRef<FControlFlowContainerBase>>& GetFinishedFlows();

private:
	template<typename OwningObjectT>
	static FControlFlow& Create_Internal(OwningObjectT* OwningObject, const FString& FlowId, bool bIsPersistent)
	{
		checkf(OwningObject, TEXT("Invalid Object!"));

		TSharedRef<TControlFlowContainer<OwningObjectT>> NewContainer = MakeShared<TControlFlowContainer<OwningObjectT>>(OwningObject, MakeShared<FControlFlow>(FlowId), FlowId);

		if (bIsPersistent)
		{
			GetPersistentFlows().Add(NewContainer);
		}
		else
		{
			GetNewlyCreatedFlows().Add(NewContainer);
		}

		NewContainer->GetControlFlow()->Reset();

		CheckNewlyCreatedFlows();

		return NewContainer->GetControlFlow().Get();
	}

private:
	template<typename OwningObjectT>
	static TSharedPtr<FControlFlow> Find_Internal(OwningObjectT* OwningObject, const FString& FlowId, TArray<TSharedRef<FControlFlowContainerBase>>& InArray)
	{
		if (ensure(OwningObject))
		{
			for (TSharedRef<FControlFlowContainerBase> ControlFlowContainer : InArray)
			{
				if (UE::Private::OwningObjectIsValid(ControlFlowContainer))
				{
					if (ControlFlowContainer->GetFlowName() == FlowId && ControlFlowContainer->OwningObjectEqualTo(OwningObject))
					{
						return ControlFlowContainer->GetControlFlow();
					}
				}
				else
				{
					CheckForInvalidFlows();
				}
			}
		}

		return nullptr;
	}

private:
	friend class FControlFlow;

	static void HandleControlFlowStartedNotification(TSharedRef<const FControlFlow> InFlow);
	static void HandleControlFlowFinishedNotification() { CheckForInvalidFlows(); }

private:
	CONTROLFLOWS_API static void CheckNewlyCreatedFlows();
	CONTROLFLOWS_API static void CheckForInvalidFlows();

private:
	static bool IterateThroughNewlyCreatedFlows(float DeltaTime);
	static bool IterateForInvalidFlows(float DeltaTime);

private:
	static FControlFlowStatics& Get();

private:
	TArray<TSharedRef<FControlFlowContainerBase>> NewlyCreatedFlows;
	TArray<TSharedRef<FControlFlowContainerBase>> PersistentFlows;
	TArray<TSharedRef<FControlFlowContainerBase>> ExecutingFlows;
	TArray<TSharedRef<FControlFlowContainerBase>> FinishedFlows;

	FTSTicker::FDelegateHandle NextFrameCheckForExecution;
	FTSTicker::FDelegateHandle NextFrameCheckForFlowCleanup;
};
