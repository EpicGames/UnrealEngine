// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Delegates/Delegate.h"

class FControlFlow;
class FControlFlowNode;
class FControlFlowNode_Task;
class FControlFlowSubTaskBase;
class FControlFlowBranch;
class FConditionalLoop;
class FConcurrentControlFlows;
class FConcurrentControlFlowBehavior;
class FTrackedActivity;

using FControlFlowNodePtr = TSharedPtr<FControlFlowNode>;
using FControlFlowNodeRef = TSharedRef<FControlFlowNode>;

DECLARE_DELEGATE_OneParam(FControlFlowWaitDelegate, FControlFlowNodeRef)
DECLARE_DELEGATE_OneParam(FControlFlowPopulator, TSharedRef<FControlFlow>)

DECLARE_DELEGATE_RetVal(bool, FControlFlowLoopComplete)
DECLARE_DELEGATE_RetVal(int32, FControlFlowBranchDecider_Legacy)

// Would be nice to not force people to use int32 for the branch key, but finding the syntax to get it to work properly proved to be more difficult than it was worth. End result is callers have to cast to an int32.
DECLARE_DELEGATE_RetVal_OneParam(int32, FControlFlowBranchDefiner, TSharedRef<FControlFlowBranch>);

enum class EConditionalLoopResult
{
	RunLoop,
	LoopFinished
};
DECLARE_DELEGATE_RetVal_OneParam(EConditionalLoopResult, FControlFlowConditionalLoopDefiner, TSharedRef<FConditionalLoop>);

DECLARE_DELEGATE_OneParam(FConcurrentFlowsDefiner, TSharedRef<FConcurrentControlFlows>);

class FControlFlowNode : public TSharedFromThis<FControlFlowNode>
{
public:
	FControlFlowNode();
	FControlFlowNode(TSharedRef<FControlFlow> ControlFlowParent, const FString& FlowNodeDebugName);

	virtual ~FControlFlowNode();

	CONTROLFLOWS_API void ContinueFlow();
	CONTROLFLOWS_API virtual void CancelFlow();

	CONTROLFLOWS_API TSharedPtr<FTrackedActivity> GetTrackedActivity() const;

	[[nodiscard]] bool HasCancelBeenRequested() const { return bCancelled; }

	virtual FString GetNodeName() const;

	CONTROLFLOWS_API void SetProfilerEventStarted();

	[[nodiscard]] const TWeakPtr<FControlFlow>& GetParent() const { return Parent; }

protected:
	friend class FControlFlow;
	friend class FControlFlowSubTaskBase;
	
	virtual void Execute() {}

	void LogExecution();

	TWeakPtr<FControlFlow> Parent = nullptr;
	FString NodeName;
	bool bCancelled = false;
	bool bWasBoundOnExecution = true;
};

class FControlFlowNode_RequiresCallback : public FControlFlowNode
{
public:
	FControlFlowNode_RequiresCallback(TSharedRef<FControlFlow> ControlFlowParent, const FString& FlowNodeDebugName);
	FControlFlowNode_RequiresCallback(TSharedRef<FControlFlow> ControlFlowParent, const FString& FlowNodeDebugName, const FControlFlowWaitDelegate& InCallback);

protected:
	virtual void Execute() override;

private:
	friend class FControlFlow;

	FControlFlowWaitDelegate Process;
};

class FControlFlowNode_SelfCompleting : public FControlFlowNode
{
public:
	FControlFlowNode_SelfCompleting(TSharedRef<FControlFlow> ControlFlowParent, const FString& FlowNodeDebugName);
	FControlFlowNode_SelfCompleting(TSharedRef<FControlFlow> ControlFlowParent, const FString& FlowNodeDebugName, const FSimpleDelegate& InCallback);

protected:
	virtual void Execute() override;

private:
	friend class FControlFlow;

	FSimpleDelegate Process;
};

DECLARE_DELEGATE_OneParam(FContolFlowTaskEvent, TSharedRef<FControlFlowNode_Task>);

class FControlFlowNode_Task : public FControlFlowNode
{
public:
	FControlFlowNode_Task(TSharedRef<FControlFlow> ControlFlowParent, TSharedRef<FControlFlowSubTaskBase> ControlFlowTask, const FString& FlowNodeDebugName);

public:
	CONTROLFLOWS_API virtual void CancelFlow() override;

protected:
	friend class FControlFlow;

	FContolFlowTaskEvent& OnExecute() const { return OnExecuteDelegate; }
	FContolFlowTaskEvent& OnCancelRequested() const { return OnCancelRequestedDelegate; }

	virtual void Execute() override;
	virtual FString GetNodeName() const override;
	void CompleteCancelFlow();

	TSharedRef<FControlFlowSubTaskBase> GetFlowTask() const { return FlowTask; }

private:
	mutable FContolFlowTaskEvent OnExecuteDelegate;
	mutable FContolFlowTaskEvent OnCancelRequestedDelegate;

	TSharedRef<FControlFlowSubTaskBase> FlowTask;
};
