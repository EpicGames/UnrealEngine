// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "ControlFlow.h" // IWYU pragma: keep

//Empty Task
class FControlFlowSubTaskBase : public TSharedFromThis<FControlFlowSubTaskBase>
{
public:
	FControlFlowSubTaskBase() {}
	FControlFlowSubTaskBase(const FString& TaskName);

	virtual ~FControlFlowSubTaskBase() = default;

	const FString& GetTaskName() const { return DebugName; }
	void SetOwningNode(TSharedRef<FControlFlowNode_Task> InNode) { OwningNode = InNode; }
	TWeakPtr<FControlFlow> GetOwningFlowForTaskNode() const;

protected:
	friend class FControlFlow;

	FSimpleDelegate& OnComplete() const { return TaskCompleteCallback; }
	FSimpleDelegate& OnCancelled() const { return TaskCancelledCallback; }

	virtual void Execute();
	virtual void Cancel();

protected:
	TWeakPtr<FControlFlowNode_Task> OwningNode;

private:
	mutable FSimpleDelegate TaskCompleteCallback;
	mutable FSimpleDelegate TaskCancelledCallback;

	FString DebugName;
};

//////////////////////////////////////////////////////////////////////////////

class FControlFlowSimpleSubTask : public FControlFlowSubTaskBase
{
public:
	FControlFlowSimpleSubTask(const FString& TaskName);

protected:
	friend class FControlFlow;

	FControlFlowPopulator& GetTaskPopulator() const { return TaskPopulator; }

	TSharedPtr<FControlFlow> GetTaskFlow() const { return TaskFlow; }

	virtual void Execute() override;
	virtual void Cancel() override;

private:
	void CompletedSubTask();
	void CancelledSubTask();

	mutable FControlFlowPopulator TaskPopulator;

	TSharedPtr<FControlFlow> TaskFlow = nullptr;
};

//////////////////////////////////////////////////////////////////////////////

class FControlFlowTask_LoopDeprecated : public FControlFlowSimpleSubTask
{
public:

	FControlFlowTask_LoopDeprecated(FControlFlowLoopComplete& TaskCompleteDelegate, const FString& TaskName);

protected:

	virtual void Execute() override;
	virtual void Cancel() override;

private:
	void CompletedLoop();
	void CancelledLoop();

	FControlFlowLoopComplete TaskCompleteDecider;
};

//////////////////////////////////////////////////////////////////////////////

class FControlFlowTask_BranchLegacy : public FControlFlowSubTaskBase
{
public:
	FControlFlowTask_BranchLegacy(FControlFlowBranchDecider_Legacy& BranchDecider, const FString& TaskName);

	/** Returns a delegate for easier syntax to bind a function for the flow to execute and then move on */
	FSimpleDelegate& QueueFunction(int32 BranchIndex, const FString& FlowNodeDebugName = TEXT(""));

	/** The flow will execute this function where the bound function takes in a ControlFlowNode. The Flow will pause until ControlFlowNode->CompleteExecution() is called. */
	FControlFlowWaitDelegate& QueueWait(int32 BranchIndex, const FString& FlowNodeDebugName = TEXT(""));

	/** Allows you to break up your flow into Smaller Tasks to help organize your flow. The flow will execute the FControlFlowPopulator only once */
	FControlFlowPopulator& QueueControlFlow(int32 BranchIndex, const FString& TaskName = TEXT(""), const FString& FlowNodeDebugName = TEXT(""));

	/** Adds a branch to your flow. The flow will use FControlFlowBranchDecider to determine which flow branch to execute */
	TSharedRef<FControlFlowTask_BranchLegacy> QueueBranch(int32 BranchIndex, FControlFlowBranchDecider_Legacy& BranchDecider, const FString& TaskName = TEXT(""), const FString& FlowNodeDebugName = TEXT(""));

	/** Adds a Loop to your flow. The flow will use FControlFlowLoopComplete - if this returns false, the flow will execute FControlFlowPopulator until true is returned */
	FControlFlowPopulator& QueueLoop(int32 BranchIndex, FControlFlowLoopComplete& LoopCompleteDelgate, const FString& TaskName = TEXT(""), const FString& FlowNodeDebugName = TEXT(""));

	template <typename... VarTypes>
	void QueueStep(int32 BranchIndex, VarTypes... Vars)
	{
		GetOrAddBranch(BranchIndex)->QueueStep(Vars...);
	}

protected:
	friend class FControlFlow;

	virtual void Execute() override;
	virtual void Cancel() override;

private:
	void HandleBranchCompleted();
	void HandleBranchCancelled();

	TSharedRef<FControlFlow> GetOrAddBranch(int32 BranchIndex);

	mutable FControlFlowBranchDecider_Legacy BranchDelegate;

	TMap<int32, TSharedRef<FControlFlow>> Branches;

	TOptional<int32> SelectedBranch;
};

//////////////////////////////////////////////////////////////////////////////

class FControlFlowTask_Branch : public FControlFlowSubTaskBase
{
public:
	friend class FControlFlow;

public:
	FControlFlowTask_Branch(const FString& TaskName)
		: FControlFlowSubTaskBase(TaskName)
	{}

	FControlFlowBranchDefiner& GetDelegate() { return BranchDelegate; }

	TSharedPtr<FTrackedActivity> Activity;

protected:
	virtual void Execute() override;
	virtual void Cancel() override;

private:

	void HandleBranchCompleted();
	void HandleBranchCancelled();

	FControlFlowBranchDefiner BranchDelegate;
	TSharedPtr<FControlFlow> SelectedBranchFlow;
};

//////////////////////////////////////////////////////////////////////////////

class FControlFlowTask_ConcurrentFlows : public FControlFlowSubTaskBase
{
public:
	friend class FControlFlow;

public:
	FControlFlowTask_ConcurrentFlows(const FString& TaskName)
		: FControlFlowSubTaskBase(TaskName)
	{}

	FConcurrentFlowsDefiner& GetDelegate() { return ConcurrentFlowDelegate; }

protected:
	virtual void Execute() override;
	virtual void Cancel() override;

private:

	void HandleConcurrentFlowsCompleted();
	void HandleConcurrentFlowsCancelled();

	FConcurrentFlowsDefiner ConcurrentFlowDelegate;
	TSharedPtr<FConcurrentControlFlows> ConcurrentFlows;
};
