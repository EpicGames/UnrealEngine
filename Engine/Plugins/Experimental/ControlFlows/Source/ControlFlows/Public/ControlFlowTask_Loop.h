// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "ControlFlowTask.h"

class FConditionalLoop;

// In contrast to, for example, iterator loops
class FControlFlowTask_ConditionalLoop : public FControlFlowSubTaskBase
{
public:
	FControlFlowTask_ConditionalLoop(const FString& TaskName)
		: FControlFlowSubTaskBase(TaskName)
	{}

	FControlFlowConditionalLoopDefiner& GetDelegate() { return ConditionalLoopDelegate; }

protected:

	virtual void Execute() override;
	virtual void Cancel() override;

private:
	void Helper_ConditionalCheck(const EConditionalLoopResult InCondition) const;

	void HandleLoopCompleted();
	void HandleLoopCancelled();
	void HandleOnNodeWasNotBoundedOnExecution();

	FControlFlowConditionalLoopDefiner ConditionalLoopDelegate;
	TSharedPtr<FConditionalLoop> ConditionalLoop;

	bool bHandleUsed = false;

	/*
	* While using the '.Loop' syntax, the object doing the bounding can suddenly be destroyed and cause a stack overflow with an infinite loop
	* 'bStopLoopingOnUnboundedDelegate' is used to prevent this
	*/
	bool bStopLoopingOnUnboundedDelegate = false;
};